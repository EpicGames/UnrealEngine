{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "ConcertInsightsVisualizer",
	"Description": "Analyses and provides visualization widgets for Concert message types in Unreal Insights.",
	"Category": "Networking",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": false,
	"Hidden": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"SupportedPrograms": [ "UnrealInsights" ],
	"Modules": [
		{
			"Name": "ConcertInsightsVisualizer",
			"Type": "EditorAndProgram",
			"LoadingPhase": "Default",
			"ProgramAllowList": [ "UnrealInsights" ]
		}
	]
}