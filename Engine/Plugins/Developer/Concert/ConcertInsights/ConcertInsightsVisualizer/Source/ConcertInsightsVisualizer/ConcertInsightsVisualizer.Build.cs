// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ConcertInsightsVisualizer : ModuleRules
{
	public ConcertInsightsVisualizer(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			});
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Projects",
				"Slate",
				"SlateCore",
				"TraceAnalysis",
				"TraceInsights",
				"TraceServices"
			});
		
		ShortName = "CrtInV";
	}
}
