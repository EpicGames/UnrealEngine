// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertInsightsStyle.h"

#include "Brushes/SlateBoxBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/StarshipCoreStyle.h"

LLM_DEFINE_TAG(ProtocolFrontendStyle);

#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush( FConcertInsightsStyle::InContent( RelativePath, ".png" ), __VA_ARGS__ )
#define IMAGE_BRUSH(RelativePath, ...) FSlateImageBrush(StyleSet->RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)
#define BOX_BRUSH(RelativePath, ... ) FSlateBoxBrush(StyleSet->RootToContentDir(RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define IMAGE_PLUGIN_BRUSH_SVG( RelativePath, ... ) FSlateVectorImageBrush(FConcertInsightsStyle::InContent(RelativePath, ".svg"), __VA_ARGS__)

namespace UE::ConcertInsightsVisualizer
{
	FString FConcertInsightsStyle::InContent(const FString& RelativePath, const ANSICHAR* Extension)
	{
		static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("ConcertInsights"))->GetContentDir();
		return (ContentDir / RelativePath) + Extension;
	}

	TSharedPtr<FSlateStyleSet > FConcertInsightsStyle::StyleSet;

	FName FConcertInsightsStyle::GetStyleSetName()
	{
		return FName(TEXT("FConcertInsightsStyle"));
	}

	void FConcertInsightsStyle::Initialize()
	{
		LLM_SCOPE_BYTAG(ProtocolFrontendStyle);
		// Only register once
		if (StyleSet.IsValid())
		{
			return;
		}

		StyleSet = MakeShared<FSlateStyleSet>(GetStyleSetName());
		StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
		StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));
		

		FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
	};

	void FConcertInsightsStyle::Shutdown()
	{
		if (StyleSet.IsValid())
		{
			FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
			ensure(StyleSet.IsUnique());
			StyleSet.Reset();
		}
	}

	TSharedPtr<class ISlateStyle> FConcertInsightsStyle::Get()
	{
		return StyleSet;
	}
}

#undef IMAGE_PLUGIN_BRUSH
#undef IMAGE_BRUSH

