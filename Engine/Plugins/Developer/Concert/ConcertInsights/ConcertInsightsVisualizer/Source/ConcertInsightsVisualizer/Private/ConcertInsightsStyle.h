// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyle.h"

namespace UE::ConcertInsightsVisualizer
{
	class FConcertInsightsStyle
	{
	public:
	
		static void Initialize();
		static void Shutdown();

		static TSharedPtr<class ISlateStyle> Get();
		static FName GetStyleSetName();
	
	private:
	
		static FString InContent(const FString& RelativePath, const ANSICHAR* Extension);
		static TSharedPtr<FSlateStyleSet> StyleSet;
	};
}