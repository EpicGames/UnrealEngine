// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertInsightsVisualizer.h"

#include "ConcertInsightsStyle.h"

#include "Features/IModularFeatures.h"
#include "Insights/ITimingViewExtender.h"

#define LOCTEXT_NAMESPACE "FConcertInsightsVisualizerModule"

namespace UE::ConcertInsightsVisualizer
{
	void FConcertInsightsVisualizerModule::StartupModule()
	{
		UE_LOGF(LogTemp, Log, "Initing FConcertInsightsVisualizerModule...");
		
		FConcertInsightsStyle::Initialize();
		RegisterInsightsExtensions();
	}

	void FConcertInsightsVisualizerModule::ShutdownModule()
	{
		UnregisterInsightsExtensions();
		FConcertInsightsStyle::Shutdown();
	}

	void FConcertInsightsVisualizerModule::RegisterInsightsExtensions()
	{
		IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &ConcertInsightsModule);
		IModularFeatures::Get().RegisterModularFeature(UE::Insights::Timing::TimingViewExtenderFeatureName, &TimingViewExtender);
	}

	void FConcertInsightsVisualizerModule::UnregisterInsightsExtensions()
	{
		IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &ConcertInsightsModule);
		IModularFeatures::Get().UnregisterModularFeature(UE::Insights::Timing::TimingViewExtenderFeatureName, &TimingViewExtender);
	}
}


#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(UE::ConcertInsightsVisualizer::FConcertInsightsVisualizerModule, ConcertInsightsVisualizer)