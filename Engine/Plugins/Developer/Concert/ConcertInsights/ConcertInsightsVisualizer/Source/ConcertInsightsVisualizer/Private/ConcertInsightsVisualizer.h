// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IConcertInsightsVisualizerModule.h"
#include "Extension/ConcertTimingViewExtender.h"
#include "Extension/ConcertTraceInsightsModule.h"

namespace UE::ConcertInsightsVisualizer
{
	class FConcertProfilerManager;
	
	class FConcertInsightsVisualizerModule : public IConcertInsightsVisualizerModule
	{
	public:

		//~ Begin IModuleInterface Interface
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;
		//~ End IModuleInterface Interface

	private:

		/** Adds Concert analyzer and provider. */
		FConcertTraceInsightsModule ConcertInsightsModule;
		/** Adds tracks to the timing view as outlined in Trace/ConcertProtocolTrace.h. */
		FConcertTimingViewExtender TimingViewExtender;

		/** Registers all Concert specific modular features with Insights. */
		void RegisterInsightsExtensions();
		void UnregisterInsightsExtensions();
	};
}
