// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertTimingViewExtender.h"

#include "Insights/ITimingViewSession.h"
#include "Insights/IUnrealInsightsModule.h"

namespace UE::ConcertInsightsVisualizer
{
	void FConcertTimingViewExtender::OnBeginSession(UE::Insights::Timing::ITimingViewSession& InSession)
	{
		if (InSession.GetName() != FInsightsManagerTabs::TimingProfilerTabId)
		{
			return;
		}

		FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
		if (!PerSessionData)
		{
			PerSessionData = &PerSessionDataMap.Add(&InSession);
			PerSessionData->SharedData = MakeUnique<FConcertTimingViewSession>();
			PerSessionData->SharedData->OnBeginSession(InSession);
		}
		else
		{
			// Should generally not enter here because OnEndSession removes it but keep it in case Insights API skips a call (can that even happen?)
			PerSessionData->SharedData->OnBeginSession(InSession);
		}
	}

	void FConcertTimingViewExtender::OnEndSession(UE::Insights::Timing::ITimingViewSession& InSession)
	{
		if (InSession.GetName() != FInsightsManagerTabs::TimingProfilerTabId)
		{
			return;
		}

		const FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
		if (PerSessionData)
		{
			PerSessionData->SharedData->OnEndSession(InSession);
		}

		PerSessionDataMap.Remove(&InSession);
	}

	void FConcertTimingViewExtender::Tick(UE::Insights::Timing::ITimingViewSession& InTimingSession, const TraceServices::IAnalysisSession& InAnalysisSession)
	{
		if (const FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InTimingSession))
		{
			PerSessionData->SharedData->Tick(InTimingSession, InAnalysisSession);
		}
	}

	void FConcertTimingViewExtender::ExtendFilterMenu(UE::Insights::Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder)
	{
		if (const FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession))
		{
			PerSessionData->SharedData->ExtendFilterMenu(InSession, InMenuBuilder);
		}
	}
}
