// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertTimingViewSession.h"

#include "Insights/ITimingViewExtender.h"
#include "Templates/UniquePtr.h"

namespace UE::ConcertInsightsVisualizer
{
	/** Keeps track of FConcertTimingViewSession per analytics session. */
	class FConcertTimingViewExtender : public UE::Insights::Timing::ITimingViewExtender
	{
	public:

		//~ Begin UE::Insights::Timing::ITimingViewExtender Interface
		virtual void OnBeginSession(UE::Insights::Timing::ITimingViewSession& InSession) override;
		virtual void OnEndSession(UE::Insights::Timing::ITimingViewSession& InSession) override;
		virtual void Tick(UE::Insights::Timing::ITimingViewSession& InTimingSession, const TraceServices::IAnalysisSession& InAnalysisSession) override;
		virtual void ExtendFilterMenu(UE::Insights::Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) override;
		//~ End UE::Insights::Timing::ITimingViewExtender Interface

	private:

		struct FPerSessionData
		{
			TUniquePtr<FConcertTimingViewSession> SharedData;
		};

		//** The data we host per-session */
		TMap<UE::Insights::Timing::ITimingViewSession*, FPerSessionData> PerSessionDataMap;
	};
}
