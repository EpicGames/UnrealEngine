// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertTimingViewSession.h"

#include "Trace/TraceAggregator.h"
#include "Track/ProtocolTrack.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Insights/ITimingViewSession.h"
#include "Insights/IUnrealInsightsModule.h"
#include "Modules/ModuleManager.h"
#include "Trace/ProtocolMultiEndpointProvider.h"

#define LOCTEXT_NAMESPACE "FConcertTimingViewSession"

namespace UE::ConcertInsightsVisualizer
{
	void FConcertTimingViewSession::OnBeginSession(UE::Insights::Timing::ITimingViewSession& InSession)
	{}

	void FConcertTimingViewSession::OnEndSession(UE::Insights::Timing::ITimingViewSession& InSession)
	{}
	
	void FConcertTimingViewSession::Tick(UE::Insights::Timing::ITimingViewSession& InTimingSession, const TraceServices::IAnalysisSession& InAnalysisSession)
	{
		const FProtocolMultiEndpointProvider* ProtocolProvider = InAnalysisSession.ReadProvider<FProtocolMultiEndpointProvider>(FProtocolMultiEndpointProvider::ProviderName);
		if (!ensure(ProtocolProvider))
		{
			return;
		}
		
		if (!ObjectTrack)
		{
			ObjectTrack = MakeShared<FProtocolTrack>(InAnalysisSession, *ProtocolProvider);
			ObjectTrack->SetVisibilityFlag(true);
			ObjectTrack->SetOrder(FTimingTrackOrder::First);
			InTimingSession.AddScrollableTrack(ObjectTrack);
			InTimingSession.InvalidateScrollableTracksOrder();
		}
	}

	void FConcertTimingViewSession::ExtendFilterMenu(UE::Insights::Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder)
	{
		if (!ObjectTrack)
		{
			return;
		}
		
		InMenuBuilder.BeginSection("ConcertTracks", LOCTEXT("ConcertTracksSection", "Concert"));
		{
			InMenuBuilder.AddMenuEntry(
				LOCTEXT("TrimPackagePaths.Label", "Show full package paths"),
				LOCTEXT("TrimPackagePaths.Tooltip", "Whether the 1st line in every sequence should show the full path to the object."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this](){ ObjectTrack->ToggleShowObjectFullPaths(); }),
					FCanExecuteAction::CreateLambda([](){ return true; }),
					FIsActionChecked::CreateLambda([this](){ return ObjectTrack->ShouldShowFullObjectPaths(); })
					),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}
		InMenuBuilder.EndSection();
	}
	
	void FConcertTimingViewSession::ConditionalInitTraceAggregator(const TraceServices::IAnalysisSession& InAnalysisSession)
	{
		if (!TraceAggregator)
		{
			IUnrealInsightsModule& InsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
			Trace::FStoreClient* StoreClient = InsightsModule.GetStoreClient();
			
			TraceAggregator = MakeShared<FTraceAggregator>(*StoreClient, InAnalysisSession.GetTraceId());
			TraceAggregator->StartAggregatedAnalysis();
		}
	}
}

#undef LOCTEXT_NAMESPACE