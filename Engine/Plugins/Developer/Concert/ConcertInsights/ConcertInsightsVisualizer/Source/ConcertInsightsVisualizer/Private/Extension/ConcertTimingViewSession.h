// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class FMenuBuilder;

namespace UE::Insights::Timing { class ITimingViewSession; }
namespace TraceServices { class IAnalysisSession; }

namespace UE::ConcertInsightsVisualizer
{
	class FProtocolTrack;
	class FTraceAggregator;

	/** Adds tracks to the timing view as outlined in Trace/ConcertProtocolTrace.h. */
	class FConcertTimingViewSession : public FNoncopyable
	{
	public:

		// Mirrors the ITimingViewExtender interface - called by FConcertTimingViewExtender
		void OnBeginSession(UE::Insights::Timing::ITimingViewSession& InSession);
		void OnEndSession(UE::Insights::Timing::ITimingViewSession& InSession);
		void Tick(UE::Insights::Timing::ITimingViewSession& InTimingSession, const TraceServices::IAnalysisSession& InAnalysisSession);
		void ExtendFilterMenu(UE::Insights::Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder);

	private:

		/** Generates the rows in the Insights UI. */
		TSharedPtr<FProtocolTrack> ObjectTrack;

		/** Traces related files and exposes their data. */
		TSharedPtr<FTraceAggregator> TraceAggregator;
		
		void ConditionalInitTraceAggregator(const TraceServices::IAnalysisSession& InAnalysisSession);
	};
}
