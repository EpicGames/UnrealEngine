// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertTraceInsightsModule.h"

#include "LogConcertInsights.h"
#include "Trace/Analysis/ProtocolEndpointAnalyzer.h"
#include "Trace/ProtocolMultiEndpointProvider.h"

namespace UE::ConcertInsightsVisualizer
{
	void FConcertTraceInsightsModule::GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo)
	{
		OutModuleInfo.Name = TEXT("ConcertTrace");
		OutModuleInfo.DisplayName = TEXT("Concert");
	}

	void FConcertTraceInsightsModule::OnAnalysisBegin(TraceServices::IAnalysisSession& Session)
	{
		UE_LOGF(LogConcertInsights, Log, "FConcertTraceInsightsModule::OnAnalysisBegin");

		const TSharedRef<FProtocolMultiEndpointProvider> Provider = MakeShared<FProtocolMultiEndpointProvider>(Session);
		Session.AddProvider(FProtocolMultiEndpointProvider::ProviderName, Provider);

		// Insights takes ownership and will call delete 
		Session.AddAnalyzer(new FProtocolEndpointAnalyzer(Session, *Provider));
	}
}
