// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/ModuleService.h"

namespace UE::ConcertInsightsVisualizer
{
	/** Adds the Concert analyzer and provider when a session starts. */
	class FConcertTraceInsightsModule : public TraceServices::IModule
	{
	public:

		//~ Begin TraceServices::IModule Interface
		virtual void GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo) override;
		virtual void OnAnalysisBegin(TraceServices::IAnalysisSession& Session) override;
		//~ End TraceServices::IModule Interface
	};
}