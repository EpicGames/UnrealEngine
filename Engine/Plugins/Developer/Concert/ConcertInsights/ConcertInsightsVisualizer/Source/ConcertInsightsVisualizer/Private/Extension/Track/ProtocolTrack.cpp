// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProtocolTrack.h"

#include "TraceServices/Containers/Timelines.h"
#include "Trace/ProtocolMultiEndpointProvider.h"

namespace UE::ConcertInsightsVisualizer
{
	FProtocolTrack::FProtocolTrack(const TraceServices::IAnalysisSession& Session, const FProtocolMultiEndpointProvider& Provider)
		: FTimingEventsTrack(TEXT("Concert"))
		, Session(Session)
		, Provider(Provider)
	{}

	void FProtocolTrack::ToggleShowObjectFullPaths()
	{
		bShouldShowFullObjectPaths = !bShouldShowFullObjectPaths;
		SetDirtyFlag();
	}

	void FProtocolTrack::BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
	{
		TraceServices::FAnalysisSessionReadScope ReadScope(Session);
		
		FBuildContext ContextInternal(Builder, Context);
		BuildSequences(ContextInternal);
	}

	void FProtocolTrack::InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const
	{
		FTimingEventsTrack::InitTooltip(InOutTooltip, InTooltipEvent);
	}

	const TSharedPtr<const ITimingEvent> FProtocolTrack::SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const
	{
		return FTimingEventsTrack::SearchEvent(InSearchParameters);
	}

	void FProtocolTrack::BuildSequences(FBuildContext& Context) const
	{
		// TODO: We should better visually separate protocols and objects, maybe by using separate tracks
		Provider.EnumerateProtocols([&](FProtocolId ProtocolId)
		{
			Provider.EnumerateObjects(ProtocolId, [&](FObjectPath ObjectPath)
			{
				const FObjectScopeInfo Object { ProtocolId, ObjectPath };
				Provider.EnumerateSequences(Context.StartTime, Context.EndTime, Object, [&](FSequenceId SequenceId)
				{
					BuildSequence(Context, { ProtocolId, ObjectPath, SequenceId });
					return TraceServices::EEventEnumerate::Continue;
				});
				return TraceServices::EEventEnumerate::Continue;
			});
			return TraceServices::EEventEnumerate::Continue;
		});
	}

	void FProtocolTrack::BuildSequence(FBuildContext& Context, const FSequenceScopeInfo& Info) const
	{
		// 1st row shows the sequence ID and
		const TOptional<FVector2d> Bounds = Provider.GetSequenceBounds(Info);
		Context.Builder.AddEvent(Bounds->X, Bounds->Y, Context.CurrentDepthOffset, *FString::Printf(TEXT("%u - %s"), Info.SequenceId, *GetObjectDisplayString(Info.ObjectPath.GetData())));
		++Context.CurrentDepthOffset;

		// 2nd row shows e.g. "Client Name" or "Transmission"
		Provider.EnumerateNetworkScopes(Context.StartTime, Context.EndTime, Info, [&](double Start, double End, const FObjectNetworkScope& NetworkScope)
		{
			const FString EventName = NetworkScope.ProcessingEndpoint
				? Provider.GetEndpointDisplayName(*NetworkScope.ProcessingEndpoint)
				: TEXT("Transmission");
			Context.Builder.AddEvent(Start, End, Context.CurrentDepthOffset, *EventName);
			return TraceServices::EEventEnumerate::Continue;
		});
		++Context.CurrentDepthOffset;

		// 3rd row and after show the where the endpoints spent CPU time
		uint32 MaxNumRows = 0;
		Provider.EnumerateEndpointsInSequence(Info, [&](FEndpointId EndpointId)
		{
			const uint32 NumRows = BuildCpuTimeline(Context, Info.MakeEndpointInfo(EndpointId));
			MaxNumRows = FMath::Max(MaxNumRows, NumRows);
			return TraceServices::EEventEnumerate::Continue;
		});
		Context.CurrentDepthOffset += MaxNumRows;
	}

	uint32 FProtocolTrack::BuildCpuTimeline(FBuildContext& Context, const FEndpointScopeInfo& Info) const
	{
		uint32 CpuTimelineNumRows = 0;
		Provider.ReadProcessingStepTimeline(Info, [&](const TraceServices::ITimeline<FObjectProcessingStep>& Timeline)
		{
			Timeline.EnumerateEventsDownSampled(Context.StartTime, Context.EndTime, Context.SecondsPerPixel,
				[&](double StartTime, double EndTime, uint32 Depth, const FObjectProcessingStep& Event)
				{
					const uint32 FinalOffset = Context.CurrentDepthOffset + Depth;
					// The top-most row is at depth 0, so +1 since we're counting the number of rows
					CpuTimelineNumRows = FMath::Max(CpuTimelineNumRows, Depth + 1); 
					
					Context.Builder.AddEvent(StartTime, EndTime, FinalOffset, Event.EventName);
					return TraceServices::EEventEnumerate::Continue;
				});
		});

		return CpuTimelineNumRows;
	}

	FString FProtocolTrack::GetObjectDisplayString(const TCHAR* ObjectPath) const
	{
		FString ObjectPathAsString = ObjectPath;
		if (bShouldShowFullObjectPaths)
		{
			return ObjectPathAsString;
		}

		const int32 FoundPosition = ObjectPathAsString.Find(TEXT("PersistentLevel."));
		if (FoundPosition == INDEX_NONE)
		{
			return ObjectPathAsString;
		}

		// PersistentLevel. has 16 characters
		const int32 ChopPosition = FoundPosition + 16;
		ObjectPathAsString.RightChopInline(ChopPosition);
		return ObjectPathAsString;
	}
}
