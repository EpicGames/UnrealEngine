// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Insights/ViewModels/TimingEventsTrack.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Trace/ProtocolMultiEndpointProvider.h"

namespace UE::ConcertInsightsVisualizer
{
	class FProtocolGraphSeries;

	class FProtocolTrack : public FTimingEventsTrack
	{
	public:

		FProtocolTrack(const TraceServices::IAnalysisSession& Session UE_LIFETIMEBOUND, const FProtocolMultiEndpointProvider& Provider UE_LIFETIMEBOUND);

		/** Toggles whether objects are displayed with full object paths or just the bit after PersistentLevel. */
		void ToggleShowObjectFullPaths();
		/** @return Whether objects are displayed with full object paths or just the bit after PersistentLevel */
		bool ShouldShowFullObjectPaths() const { return bShouldShowFullObjectPaths; }
		
		//~ Begin FTimingEventsTrack Interface
		virtual void BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;
		virtual void InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const override;
		virtual const TSharedPtr<const ITimingEvent> SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const override;
		//~ End FTimingEventsTrack Interface

	private:

		struct FBuildContext
		{
			ITimingEventsTrackDrawStateBuilder& Builder;
			const ITimingTrackUpdateContext& TimingContext;
			
			const double StartTime = GetViewport().GetStartTime();
			const double EndTime = GetViewport().GetEndTime();
			const double SecondsPerPixel = 1.0 / GetViewport().GetScaleX();
			
			uint32 CurrentDepthOffset = 0;

			FBuildContext(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
				: Builder(Builder)
				, TimingContext(Context)
			{}

			const FTimingTrackViewport& GetViewport() const { return TimingContext.GetViewport(); }
		};

		const TraceServices::IAnalysisSession& Session;
		const FProtocolMultiEndpointProvider& Provider;

		/** Whether the 1st row of every sequence should show the full object path or just the part behind .PersistentLevel */
		bool bShouldShowFullObjectPaths = false;

		void BuildSequences(FBuildContext& Context) const;
		void BuildSequence(FBuildContext& Context, const FSequenceScopeInfo& Info) const;
		uint32 BuildCpuTimeline(
			FBuildContext& Context,
			const FEndpointScopeInfo& Info
			) const;

		/** Gets the final display string of ObjectPath (given the value of bShouldShowFullObjectPaths). */
		FString GetObjectDisplayString(const TCHAR* ObjectPath) const;
	};
}


