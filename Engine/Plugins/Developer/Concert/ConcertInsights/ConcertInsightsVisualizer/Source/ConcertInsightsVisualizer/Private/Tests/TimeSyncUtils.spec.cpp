// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "Util/TimeSyncUtils.h"

#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"

namespace UE::ConcertInsightsVisualizer::Tests
{
	
	BEGIN_DEFINE_SPEC(FTimeSyncUtilsSpec, "Editor.Concert.Insights", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	
	END_DEFINE_SPEC(FTimeSyncUtilsSpec);

	void FTimeSyncUtilsSpec::Define()
	{
		BeforeEach([this]()
		{
			
		});

		Describe("Convert time correctly", [this]()
		{
			/* See documentation of ConvertSourceToTargetTime
			 * Global	0123456789_
			 * Target	[Init567x9]
			 * Source	-[1Init6y8]
			 */
			It("For example documentation case", [this]()
			{
				const FDateTime TargetInitUtc(2024, 2, 22, 12, 0, 1);
				const FDateTime SourceInitUtc(2024, 2, 22, 12, 0, 3);
				constexpr double TargetInitTime = 1;
				constexpr double SourceInitTime = 2;
				constexpr double SourceTime = 7; // = y from above

				// Compute x from tomorrow
				const double ConvertedTime = TimeSyncUtils::ConvertSourceToTargetTime(
					TargetInitUtc, SourceInitUtc, TargetInitTime, SourceInitTime, SourceTime
					);
				// Inverse: Compute y from x
				const double InverseConvertedTime = TimeSyncUtils::ConvertSourceToTargetTime(
					SourceInitUtc, TargetInitUtc, SourceInitTime, TargetInitTime, ConvertedTime
					);
				
				TestEqual(TEXT("Docu use case time"), ConvertedTime, 8.0);
				TestEqual(TEXT("Inverse docu use case time"), InverseConvertedTime, SourceTime);
			});
			
			It("When target 30 seconds later", [this]()
			{

			});

			It("When target 30 seconds earlier", [this]()
			{
				
			});
		});
	}
}
