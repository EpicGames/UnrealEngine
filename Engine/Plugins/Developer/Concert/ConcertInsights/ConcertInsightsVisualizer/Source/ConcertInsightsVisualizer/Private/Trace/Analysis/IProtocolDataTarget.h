// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::ConcertInsightsVisualizer
{
	struct FInitMessage;
	struct FObjectSinkMessage;
	struct FObjectTraceBeginMessage;
	struct FObjectTraceEndMessage;
	struct FObjectTransmissionReceiveMessage;
	struct FObjectTransmissionStartMessage;
	
	/**
	 * Target for receiving data from FProtocolEndpointAnalyzer.
	 * 
	 * This abstracts what is done with the data, which is important for aggregation:
	 * - The main .utrace, which is the one that is technically open, is processed on the main thread and directly inserted into FProtocolMultiEndpointProvider.
	 * - The aggregated .utrace files are processed on a separate thread. The analyzed data must be synchronized with the main thread which is controlled by
	 * FProtocolDataQueue. FProtocolMultiEndpointProvider will dequeue ready data from FProtocolDataQueue on Tick.
	 * 
	 */
	class IProtocolDataTarget
	{
	public:
		
		virtual void AppendInit(FInitMessage Message) = 0;
		virtual void AppendObjectTraceBegin(FObjectTraceBeginMessage Message) = 0;
		virtual void AppendObjectTraceEnd(FObjectTraceEndMessage Message) = 0;
		virtual void AppendObjectTransmissionStart(FObjectTransmissionStartMessage Message) = 0;
		virtual void AppendObjectTransmissionReceive(FObjectTransmissionReceiveMessage Message) = 0;
		virtual void AppendObjectSink(FObjectSinkMessage Message) = 0;

		virtual ~IProtocolDataTarget() = default;
	};
}