// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IProtocolDataTarget.h"
#include "Trace/Messages/EMessageType.h"
#include "Trace/Messages/InitMessage.h"
#include "Trace/Messages/ObjectSinkMessage.h"
#include "Trace/Messages/ObjectTraceMessage.h"
#include "Trace/Messages/ObjectTransmissionReceiveMessage.h"
#include "Trace/Messages/ObjectTransmissionStartMessage.h"

#include "Containers/Queue.h"

#include <type_traits>

namespace UE::ConcertInsightsVisualizer
{
	static_assert(std::is_trivial_v<FInitMessage>, "FInitMessage must be trivial for the FProtocolQueuedItem::FMessage union.");
	static_assert(std::is_trivial_v<FObjectTraceBeginMessage>, "FObjectTraceBeginMessage must be trivial for the FProtocolQueuedItem::FMessage union.");
	static_assert(std::is_trivial_v<FObjectTraceEndMessage>, "FObjectTraceEndMessage must be trivial for the FProtocolQueuedItem::FMessage union.");
	static_assert(std::is_trivial_v<FObjectTransmissionStartMessage>, "FObjectTransmissionStartMessage must be trivial for the FProtocolQueuedItem::FMessage union.");
	static_assert(std::is_trivial_v<FObjectTransmissionReceiveMessage>, "FObjectTransmissionReceiveMessage must be trivial for the FProtocolQueuedItem::FMessage union.");
	static_assert(std::is_trivial_v<FObjectSinkMessage>, "FObjectSinkMessage must be trivial for the FProtocolQueuedItem::FMessage union.");
	
	struct FProtocolQueuedItem
	{
		union FMessage
		{
			FInitMessage Init;
			FObjectTraceBeginMessage ObjectTraceBegin;
			FObjectTraceEndMessage ObjectTraceEnd;
			FObjectTransmissionStartMessage TransmissionStart;
			FObjectTransmissionReceiveMessage TransmissionReceive;
			FObjectSinkMessage Sink;

			FMessage() = default;
			FMessage(const FInitMessage& InitMessage) : Init(InitMessage) {}
			FMessage(const FObjectTraceBeginMessage& ObjectTraceBegin) : ObjectTraceBegin(ObjectTraceBegin) {}
			FMessage(const FObjectTraceEndMessage& ObjectTraceEnd) : ObjectTraceEnd(ObjectTraceEnd) {}
			FMessage(const FObjectTransmissionStartMessage& TransmissionStart) : TransmissionStart(TransmissionStart) {}
			FMessage(const FObjectTransmissionReceiveMessage& TransmissionEnd) : TransmissionReceive(TransmissionEnd) {}
			FMessage(const FObjectSinkMessage& Sink) : Sink(Sink) {}
			
		} Message;

		/** Indicates which data in Message has been written to. */
		EMessageType Type;

		FProtocolQueuedItem() : Message(), Type(EMessageType::None) {}
		template<typename TMessage>
		FProtocolQueuedItem(const TMessage& Message) : Message(Message), Type(TMessage::Type()) {}
	};
	
	/**
	 * Used to synchronize analyzed data between two threads.
	 * 
	 * Aggregated .utrace files are analyzed by FProtocolEndpointAnalyzer on a separate thread.
	 * FProtocolMultiEndpoint will load the data every tick on the main thread.
	 */
	class FProtocolDataQueue : public IProtocolDataTarget
	{
	public:

		/** Queue of messages in the call order of the below functions. */
		TQueue<FProtocolQueuedItem> MessageQueue;
		
		//~ Begin IProtocolDataTarget Interface
		virtual void AppendInit(FInitMessage Message) override { MessageQueue.Enqueue(FProtocolQueuedItem(Message)); }
		virtual void AppendObjectTraceBegin(FObjectTraceBeginMessage Message) override { MessageQueue.Enqueue(FProtocolQueuedItem(Message)); }
		virtual void AppendObjectTraceEnd(FObjectTraceEndMessage Message) override { MessageQueue.Enqueue(FProtocolQueuedItem(Message)); }
		virtual void AppendObjectTransmissionStart(FObjectTransmissionStartMessage Message) override { MessageQueue.Enqueue(FProtocolQueuedItem(Message)); }
		virtual void AppendObjectTransmissionReceive(FObjectTransmissionReceiveMessage Message) override { MessageQueue.Enqueue(FProtocolQueuedItem(Message)); }
		virtual void AppendObjectSink(FObjectSinkMessage Message) override { MessageQueue.Enqueue(FProtocolQueuedItem(Message)); }
		//~ End IProtocolDataTarget Interface
	};
}
