// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProtocolEndpointAnalyzer.h"

#include "IProtocolDataTarget.h"
#include "LogConcertInsights.h"
#include "Trace/Messages/InitMessage.h"
#include "Trace/Messages/ObjectSinkMessage.h"
#include "Trace/Messages/ObjectTraceMessage.h"
#include "Trace/Messages/ObjectTransmissionReceiveMessage.h"
#include "Trace/Messages/ObjectTransmissionStartMessage.h"

namespace UE::ConcertInsightsVisualizer
{
	FProtocolEndpointAnalyzer::FProtocolEndpointAnalyzer(TraceServices::IAnalysisSession& Session, IProtocolDataTarget& DataTarget)
		: Session(Session)
		, DataTarget(DataTarget)
	{}

	void FProtocolEndpointAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
	{
		FInterfaceBuilder& Builder = Context.InterfaceBuilder;
		
		Builder.RouteEvent(RouteId_Init, "ConcertLogger", "Init");
		Builder.RouteEvent(RouteId_ObjectTraceBegin, "ConcertLogger", "ObjectTraceBegin");
		Builder.RouteEvent(RouteId_ObjectTraceEnd, "ConcertLogger", "ObjectTraceEnd");
		Builder.RouteEvent(RouteId_ObjectTransmissionStart, "ConcertLogger", "ObjectTransmissionStart");
		Builder.RouteEvent(RouteId_ObjectTransmissionReceive, "ConcertLogger", "ObjectTransmissionReceive");
		Builder.RouteEvent(RouteId_ObjectSink, "ConcertLogger", "ObjectSink");
	}

	bool FProtocolEndpointAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
	{
		LLM_SCOPE_BYNAME(TEXT("Insights/FProtocolEndpointAnalyzer"));
		
		const FEventData& EventData = Context.EventData;
		const FEventTime& EventTime = Context.EventTime;
		switch (RouteId)
		{
		case RouteId_Init:
		{
			DataTarget.AppendInit(FInitMessage(EventData, EventTime, Session));
			break;
		}
		case RouteId_ObjectTraceBegin:
		{
			DataTarget.AppendObjectTraceBegin(FObjectTraceBeginMessage(EventData, EventTime, Session));
			break;
		}
		case RouteId_ObjectTraceEnd:
		{
			DataTarget.AppendObjectTraceEnd(FObjectTraceEndMessage(EventData, EventTime, Session));
			break;
		}
		case RouteId_ObjectTransmissionStart:
		{
			DataTarget.AppendObjectTransmissionStart(FObjectTransmissionStartMessage(EventData, EventTime, Session));
			break;
		}
		case RouteId_ObjectTransmissionReceive:
		{
			DataTarget.AppendObjectTransmissionReceive(FObjectTransmissionReceiveMessage(EventData, EventTime, Session));
			break;
		}
		case RouteId_ObjectSink:
		{
			DataTarget.AppendObjectSink(FObjectSinkMessage(EventData, EventTime, Session));
			break;
		}
			
		default:
		UE_LOGF(LogConcertInsights, Warning, "Unknown RouteId %d", RouteId);
		}

		return true;
	}
}
