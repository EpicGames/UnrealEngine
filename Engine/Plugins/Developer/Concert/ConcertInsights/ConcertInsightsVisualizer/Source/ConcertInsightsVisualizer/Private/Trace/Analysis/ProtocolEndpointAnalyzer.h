// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"

namespace TraceServices { class IAnalysisSession; }

namespace UE::ConcertInsightsVisualizer
{
	class IProtocolDataTarget;

	/** Processes raw .utrace data, structures it into messages, and passes them to a IProtocolDataTarget. */
	class FProtocolEndpointAnalyzer : public Trace::IAnalyzer
	{
	public:

		/**
		 * @param Session Caller is responsible for ensuring it outlives the constructed object.
		 * @param DataTarget Caller is responsible for ensuring it outlives the constructed object.
		 */
		FProtocolEndpointAnalyzer(TraceServices::IAnalysisSession& Session UE_LIFETIMEBOUND, IProtocolDataTarget& DataTarget UE_LIFETIMEBOUND);

		//~ Begin Trace::IAnalyzer Interface
		virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
		virtual void OnAnalysisEnd() override {}
		virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;
		//~ End Trace::IAnalyzer Interface
		
	private:
		
		enum : uint16
		{
			RouteId_Init,
			RouteId_ObjectTraceBegin,
			RouteId_ObjectTraceEnd,
			RouteId_ObjectTransmissionStart,
			RouteId_ObjectTransmissionReceive,
			RouteId_ObjectSink
		};

		/** The session this provider operates on. */
		TraceServices::IAnalysisSession& Session;

		/** Stores the analyzed data. */
		IProtocolDataTarget& DataTarget;
	};
}

