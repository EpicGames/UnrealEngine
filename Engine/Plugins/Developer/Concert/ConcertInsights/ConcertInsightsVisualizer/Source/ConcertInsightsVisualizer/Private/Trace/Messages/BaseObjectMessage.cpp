// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseObjectMessage.h"

#include "TraceServices/Model/AnalysisSession.h"

namespace UE::ConcertInsightsVisualizer
{
	namespace ObjectTraceMessage
	{
		static const TCHAR* GetObjectPath(
			const Trace::IAnalyzer::FEventData& EventData,
			TraceServices::IAnalysisSession& Session UE_LIFETIMEBOUND
			)
		{
			FString ObjectPath;
			EventData.GetString("ObjectPath", ObjectPath);
			return Session.StoreString(*ObjectPath);
		}
	}
	
	FBaseObjectMessage::FBaseObjectMessage(
		const Trace::IAnalyzer::FEventData& EventData,
		TraceServices::IAnalysisSession& Session UE_LIFETIMEBOUND
		)
		: Protocol(EventData.GetValue<FProtocolId>("Protocol"))
		, SequenceId(EventData.GetValue<FSequenceId>("SequenceId"))
		, ObjectPath(ObjectTraceMessage::GetObjectPath(EventData, Session))
	{}
}
