// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Types/ProtocolId.h"
#include "Trace/Types/SequenceId.h"

#include "HAL/Platform.h"
#include "Trace/Analyzer.h"
#include "UObject/SoftObjectPath.h"

#include <type_traits>

namespace TraceServices { class IAnalysisSession; }

namespace UE::ConcertInsightsVisualizer
{
	/** Shared data for messages about objects. */
	struct FBaseObjectMessage
	{
		FProtocolId Protocol;
		FSequenceId SequenceId;
		/** Points to string stored in IAnalysisSession::StoreString */
		const TCHAR* ObjectPath;
		
		FBaseObjectMessage() = default;
		FBaseObjectMessage(const Trace::IAnalyzer::FEventData& EventData, TraceServices::IAnalysisSession& Session UE_LIFETIMEBOUND);

		FSoftObjectPath GetSoftObjectPath() const { return FSoftObjectPath(ObjectPath); }
	};
	static_assert(std::is_trivial_v<FBaseObjectMessage>, "FBaseObjectMessage must be trivial for the FProtocolQueuedItem::FMessage union.");
}
