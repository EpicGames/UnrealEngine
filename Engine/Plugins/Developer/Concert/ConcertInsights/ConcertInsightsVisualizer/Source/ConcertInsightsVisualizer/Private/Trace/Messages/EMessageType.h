// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::ConcertInsightsVisualizer
{
	enum class EMessageType : uint8
	{
		None,
		Init,
		ObjectTraceBegin,
		ObjectTraceEnd,
		ObjectTransmissionStart,
		ObjectTransmissionReceive,
		ObjectSink
	};
}

