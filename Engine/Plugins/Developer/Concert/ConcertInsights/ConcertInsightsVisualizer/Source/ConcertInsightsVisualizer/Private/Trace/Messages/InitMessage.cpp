// Copyright Epic Games, Inc. All Rights Reserved.

#include "InitMessage.h"

#include "TraceServices/Model/AnalysisSession.h"

namespace UE::ConcertInsightsVisualizer
{
	namespace InitMessage
	{
		static const TCHAR* GetClientDisplayName(
			const Trace::IAnalyzer::FEventData& EventData,
			TraceServices::IAnalysisSession& Session UE_LIFETIMEBOUND
			)
		{
			const bool bHasDisplayName = EventData.GetValue<bool>("HasDisplayName");
			FString ClientDisplayName;
			EventData.GetString("ClientDisplayName", ClientDisplayName);
			return bHasDisplayName ? Session.StoreString(ClientDisplayName) : nullptr;
		}
	}

	FInitMessage::FInitMessage(const Trace::IAnalyzer::FEventData& EventData, const Trace::IAnalyzer::FEventTime& EventTime, TraceServices::IAnalysisSession& Session)
		: Version(static_cast<ConcertTrace::EConcertTraceVersion>(EventData.GetValue<uint8>("Version")))
		, bHasEndpointId(EventData.GetValue<bool>("HasEndpointId"))
		, EndpointId_A(EventData.GetValue<uint32>("EndpointId_A"))
		, EndpointId_B(EventData.GetValue<uint32>("EndpointId_B"))
		, EndpointId_C(EventData.GetValue<uint32>("EndpointId_C"))
		, EndpointId_D(EventData.GetValue<uint32>("EndpointId_D"))
		, TraceInitUtc_Ticks(EventData.GetValue<int64>("DateTimeTicks"))
		, StartTime(EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle")))
		, ClientDisplayName(InitMessage::GetClientDisplayName(EventData, Session))
		, bIsServer(EventData.GetValue<bool>("IsServer"))
	{}
}
