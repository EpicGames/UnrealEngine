// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EMessageType.h"

#include "HAL/Platform.h"
#include "Misc/DateTime.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "Trace/Analyzer.h"

#include <type_traits>

namespace UE::ConcertTrace { enum class EConcertTraceVersion : uint8; }
namespace TraceServices { class IAnalysisSession; }

namespace UE::ConcertInsightsVisualizer
{
	struct FInitMessage
	{
		FInitMessage() = default;
		FInitMessage(const Trace::IAnalyzer::FEventData& EventData, const Trace::IAnalyzer::FEventTime& EventTime, TraceServices::IAnalysisSession& Session UE_LIFETIMEBOUND);
		
		static constexpr EMessageType Type() { return EMessageType::Init; }

		ConcertTrace::EConcertTraceVersion GetVersion() const { return Version; }
		TOptional<FGuid> GetEndpointId() const { return bHasEndpointId ? FGuid(EndpointId_A, EndpointId_B, EndpointId_C, EndpointId_D) : TOptional<FGuid>(); }
		/** The UTC time reported on the sending machine when the init event was started. Used to correlate cycles across multiple machines. */
		FDateTime GetTraceInitTimeUtc() const { return FDateTime(TraceInitUtc_Ticks); }
		/** Seconds since start of the trace. */
		double GetStartTime() const { return StartTime; }
		/** Points to string stored in IAnalysisSession::StoreString */
		TOptional<const TCHAR*> GetClientDisplayName() const { return ClientDisplayName; }
		bool IsServer() const { return bIsServer; }

	private: // This struct is a bit more complicated so let's force use of the above getters
		
		ConcertTrace::EConcertTraceVersion Version;

		bool bHasEndpointId;
		uint32 EndpointId_A;
		uint32 EndpointId_B;
		uint32 EndpointId_C;
		uint32 EndpointId_D;
		
		/** The UTC time reported on the sending machine when the init event was started. Used to correlate cycles across multiple machines. */
		int64 TraceInitUtc_Ticks;
		/** Seconds since start of the trace. */
		double StartTime;
		
		/** Points to string stored in IAnalysisSession::StoreString */
		const TCHAR* ClientDisplayName;
		bool bIsServer;
	};

	static_assert(std::is_trivial_v<FInitMessage>, "FInitMessage must be unique so it can be put into a union.");
}
