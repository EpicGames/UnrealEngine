// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseObjectMessage.h"
#include "EMessageType.h"

#include "HAL/Platform.h"
#include "Trace/Analyzer.h"

#include <type_traits>

namespace UE::ConcertInsightsVisualizer
{
	struct FObjectSinkMessage : FBaseObjectMessage
	{
		/** The time at which the message was generated. Seconds since start of trace. */
		double Time;
		
		/** Points to string stored in IAnalysisSession::StoreString */
		const TCHAR* SinkName;
		
		FObjectSinkMessage() = default;
		FObjectSinkMessage(
			const Trace::IAnalyzer::FEventData& EventData,
			const Trace::IAnalyzer::FEventTime& EventTime,
			TraceServices::IAnalysisSession& Session UE_LIFETIMEBOUND
			);
		
		static constexpr EMessageType Type() { return EMessageType::ObjectSink; }
	};
	
	static_assert(std::is_trivial_v<FObjectSinkMessage>, "FObjectSinkMessage must be unique so it can be put into a union.");
}
