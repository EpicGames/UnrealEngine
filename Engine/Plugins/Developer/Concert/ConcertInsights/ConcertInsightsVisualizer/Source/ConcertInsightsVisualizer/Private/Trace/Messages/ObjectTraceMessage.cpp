// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectTraceMessage.h"

#include "TraceServices/Model/AnalysisSession.h"

namespace UE::ConcertInsightsVisualizer
{
	namespace ObjectTraceMessage
	{
		static const TCHAR* GetEventName(
			const Trace::IAnalyzer::FEventData& EventData,
			TraceServices::IAnalysisSession& Session UE_LIFETIMEBOUND
			)
		{
			FString EventName;
			EventData.GetString("EventName", EventName);
			return Session.StoreString(*EventName);
		}
	}
	
	FObjectTraceMessage::FObjectTraceMessage(
		const Trace::IAnalyzer::FEventData& EventData,
		const Trace::IAnalyzer::FEventTime& EventTime,
		TraceServices::IAnalysisSession& Session UE_LIFETIMEBOUND
		)
		: FBaseObjectMessage(EventData, Session)
		, Time(EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle")))
		, EventName(ObjectTraceMessage::GetEventName(EventData, Session))
	{}
}
