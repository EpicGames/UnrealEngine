// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseObjectMessage.h"
#include "EMessageType.h"

#include "HAL/Platform.h"
#include "Trace/Analyzer.h"

namespace UE::ConcertInsightsVisualizer
{
	struct FObjectTraceMessage : FBaseObjectMessage
	{
		/** Time of the event. Seconds since start of trace. */
		double Time;
		const TCHAR* EventName;
		
		FObjectTraceMessage() = default;
		FObjectTraceMessage(
			const Trace::IAnalyzer::FEventData& EventData,
			const Trace::IAnalyzer::FEventTime& EventTime,
			TraceServices::IAnalysisSession& Session UE_LIFETIMEBOUND
			);
	};
	static_assert(std::is_trivial_v<FObjectTraceMessage>, "FObjectTraceMessage must be unique so it can be put into a union.");

	struct FObjectTraceBeginMessage : FObjectTraceMessage
	{
		FObjectTraceBeginMessage() = default;
		FObjectTraceBeginMessage(
			const Trace::IAnalyzer::FEventData& EventData,
			const Trace::IAnalyzer::FEventTime& EventTime,
			TraceServices::IAnalysisSession& Session UE_LIFETIMEBOUND
			) : FObjectTraceMessage(EventData, EventTime, Session)
		{}
		
		static constexpr EMessageType Type() { return EMessageType::ObjectTraceBegin; }
	};
	static_assert(std::is_trivial_v<FObjectTraceBeginMessage>, "FObjectTraceBeginMessage must be trivial for the FProtocolQueuedItem::FMessage union.");
	
	struct FObjectTraceEndMessage : FObjectTraceMessage
	{
		FObjectTraceEndMessage() = default;
		FObjectTraceEndMessage(
			const Trace::IAnalyzer::FEventData& EventData,
			const Trace::IAnalyzer::FEventTime& EventTime,
			TraceServices::IAnalysisSession& Session UE_LIFETIMEBOUND
			) : FObjectTraceMessage(EventData, EventTime, Session)
		{}
		
		static constexpr EMessageType Type() { return EMessageType::ObjectTraceEnd; }
	};
	static_assert(std::is_trivial_v<FObjectTraceEndMessage>, "FObjectTraceEndMessage must be unique so it can be put into a union.");
}
