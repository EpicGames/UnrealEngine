// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectTransmissionReceiveMessage.h"

namespace UE::ConcertInsightsVisualizer
{
	FObjectTransmissionReceiveMessage::FObjectTransmissionReceiveMessage(
		const Trace::IAnalyzer::FEventData& EventData,
		const Trace::IAnalyzer::FEventTime& EventTime,
		TraceServices::IAnalysisSession& Session UE_LIFETIMEBOUND
		)
		: FBaseObjectMessage(EventData, Session)
		, Time(EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle")))
	{}
}
