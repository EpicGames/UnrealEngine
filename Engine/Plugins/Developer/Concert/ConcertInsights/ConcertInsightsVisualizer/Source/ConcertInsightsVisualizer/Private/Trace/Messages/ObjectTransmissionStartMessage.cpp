// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectTransmissionStartMessage.h"

namespace UE::ConcertInsightsVisualizer
{
	namespace ObjectTransmissionStartMessage
	{
		static FGuid GetDestinationEndpointId(const Trace::IAnalyzer::FEventData& EventData)
		{
			const uint32 EndpointId_A = EventData.GetValue<uint32>("DestEndpointId_A");
			const uint32 EndpointId_B = EventData.GetValue<uint32>("DestEndpointId_B");
			const uint32 EndpointId_C = EventData.GetValue<uint32>("DestEndpointId_C");
			const uint32 EndpointId_D = EventData.GetValue<uint32>("DestEndpointId_D");
			return FGuid(EndpointId_A, EndpointId_B, EndpointId_C, EndpointId_D);
		}
	}
	
	FObjectTransmissionStartMessage::FObjectTransmissionStartMessage(
		const Trace::IAnalyzer::FEventData& EventData,
		const Trace::IAnalyzer::FEventTime& EventTime,
		TraceServices::IAnalysisSession& Session UE_LIFETIMEBOUND
		)
		: FBaseObjectMessage(EventData, Session)
		, DestinationEndpointId_A(EventData.GetValue<uint32>("DestEndpointId_A"))
		, DestinationEndpointId_B(EventData.GetValue<uint32>("DestEndpointId_B"))
		, DestinationEndpointId_C(EventData.GetValue<uint32>("DestEndpointId_C"))
		, DestinationEndpointId_D(EventData.GetValue<uint32>("DestEndpointId_D"))
		, Time(EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle")))
	{}
}
