// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseObjectMessage.h"
#include "EMessageType.h"

#include "HAL/Platform.h"
#include "Trace/Analyzer.h"

#include <type_traits>

namespace UE::ConcertInsightsVisualizer
{
	struct FObjectTransmissionStartMessage : FBaseObjectMessage
	{
		FObjectTransmissionStartMessage() = default;
		FObjectTransmissionStartMessage(
			const Trace::IAnalyzer::FEventData& EventData,
			const Trace::IAnalyzer::FEventTime& EventTime,
			TraceServices::IAnalysisSession& Session UE_LIFETIMEBOUND
			);
		
		static constexpr EMessageType Type() { return EMessageType::ObjectTransmissionStart; }

		FGuid GetDestinationEdnpointId() const { return FGuid(DestinationEndpointId_A, DestinationEndpointId_B, DestinationEndpointId_C, DestinationEndpointId_D); }
		double GetTime() const { return Time; }

	private: // This struct is a bit more complicated so let's force use of the above getters

		// Used to form FGuid for the destination endpoint
		int64 DestinationEndpointId_A;
		int64 DestinationEndpointId_B;
		int64 DestinationEndpointId_C;
		int64 DestinationEndpointId_D;
		
		/** The time at which the message was generated. Seconds since start of trace. */
		double Time;
	};
	static_assert(std::is_trivial_v<FObjectTransmissionStartMessage>, "FObjectTransmissionStartMessage must be unique so it can be put into a union.");
}
