// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Types/EndpointId.h"

#include "HAL/Platform.h"
#include "Misc/Optional.h"

namespace UE::ConcertInsightsVisualizer
{
	
	/** Nested right under FObjectSequence. Indicates on which client the object was (or whether it was in transit). */
	struct FObjectNetworkScope
	{
		/**
		 * The endpoint that corresponds to this scope, if any.
		 * Unset means that this is a network transit of the packet from one to another endpoint.
		 * FEndpointId can be used to resolve the display name in FProtocolMultiEndpointProvider.
		 */
		TOptional<FEndpointId> ProcessingEndpoint;

		bool IsEndpoint() const { return ProcessingEndpoint.IsSet(); }
		bool IsTransportAcrossNetwork() const { return !IsEndpoint(); }
	};
}
