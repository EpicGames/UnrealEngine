// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::ConcertInsightsVisualizer
{
	/** Nested under FObjectNetworkScope. Describes what an endpoint spent CPU time on while processing an object in a sequence. */
	struct FObjectProcessingStep
	{
		/** Name of the processing step */
		const TCHAR* EventName;
	};
}