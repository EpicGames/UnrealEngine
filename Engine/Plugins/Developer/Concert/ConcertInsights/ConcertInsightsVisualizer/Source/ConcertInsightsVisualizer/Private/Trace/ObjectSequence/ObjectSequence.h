// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Types/SequenceId.h"

namespace UE::ConcertInsightsVisualizer
{
	/**
	 * A sequence is a time range in which an object was processed across multiple endpoints.
	 * This data is displayed top-most event.
	 */
	struct FObjectSequence
	{
		/** ID for the object update. */
		FSequenceId SequenceId;
	};
}
