// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProtocolMultiEndpointProvider.h"

#include "LogConcertInsights.h"

#include "Algo/AnyOf.h"
#include "HAL/IConsoleManager.h"
#include "Insights/IUnrealInsightsModule.h"
#include "Modules/ModuleManager.h"
#include "Util/TimeSyncUtils.h"

namespace UE::ConcertInsightsVisualizer
{
	namespace Private
	{
		/** Given a time window of Insights, should the given scope be displayed? */
		static bool ShouldDisplayScopeInWindow(double WindowStart, double WindowEnd, double ScopeStart, double ScopeEnd)
		{
			const bool bDoesWindowIncludeStart = WindowStart <= ScopeStart && ScopeStart <= WindowEnd;
			const bool bDoesWindowIncludeEnd = WindowStart <= ScopeEnd && ScopeEnd <= WindowEnd;
			const bool bDoesSequenceIncludeWindow = ScopeStart <= WindowStart && WindowEnd <= ScopeEnd;
			
			const bool bShouldDisplayScope = bDoesWindowIncludeStart || bDoesWindowIncludeEnd || bDoesSequenceIncludeWindow;
			return bShouldDisplayScope;
		}

		static void LogInitMessage(const TCHAR* Preamble, FEndpointId TraceFileId, const FInitMessage& Init)
		{
			UE_LOGF(LogConcertInsights, Log, "%ls %llu: bIsServer: %d, DisplayName: %ls, UTC: %ls",
				Preamble,
				TraceFileId,
				Init.IsServer(),
				Init.GetClientDisplayName() ? *Init.GetClientDisplayName() : TEXT("none"),
				*Init.GetTraceInitTimeUtc().ToString(TEXT("%Y-%m-%d_%H-%Mm-%Ss-%sms"))
				);
		}

		// Read SubscribeTickerIfEnabled for more info.
		static TAutoConsoleVariable<bool> CVarEnableGameThreadAggregation(
			TEXT("Insights.Concert.EnableGameThreadAggregation"),
			false,
			TEXT("Whether aggregation of the trace files should occur on the game thread (can freeze UI)."),
			ECVF_Default
		);

		template<typename TCreateDelegateLambda>
		static FTSTicker::FDelegateHandle SubscribeTickerIfEnabled(TCreateDelegateLambda&& CreateDelegateLambda)
		{
			if (CVarEnableGameThreadAggregation.GetValueOnAnyThread())
			{
				return FTSTicker::GetCoreTicker().AddTicker(CreateDelegateLambda());
			}

			// This is an experimental plugin.
			// Aggregation currently is only implemented to occur on the game thread, which can freeze the program.
			// This plugin may get enabled but not used (e.g. Insights devs iterating on the API may enable this plugin to check it compiles, etc. but not actually use it at runtime).
			// Do not slow down the program for such cases - so by default this CVar is disabled.
			// Since this plugin is experimental and not being actively worked on, we're not going to implement performant aggregation right now.
			// The proper solution would be as follows:
			// 1. FProtocolMultiEndpointProvider::ProcessGeneratedTraceData should occur on a separate thread because it takes long
			// 2. That new thread and the game thread should be synchronized with custom read / write locks, we'd have to introduce, as well.
			UE_LOGF(LogConcertInsights, Warning, "ConcertInsights will not work because console variable Insights.Concert.EnableGameThreadAggregation is set to false.");
			return FTSTicker::FDelegateHandle{};
		}
		
	}
	
	const FName FProtocolMultiEndpointProvider::ProviderName = TEXT("FProtocolMultiEndpointProvider");
	
	FProtocolMultiEndpointProvider::FProtocolMultiEndpointProvider(TraceServices::IAnalysisSession& Session UE_LIFETIMEBOUND)
		: Session(Session)
		, Aggregator(*FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights").GetStoreClient(), GetMainTraceId())
		, TickHandle(Private::SubscribeTickerIfEnabled([this]{ return FTickerDelegate::CreateRaw(this, &FProtocolMultiEndpointProvider::ProcessGeneratedTraceData); }))
	{}

	FProtocolMultiEndpointProvider::~FProtocolMultiEndpointProvider()
	{
		if (TickHandle.IsValid())
		{
			FTSTicker::RemoveTicker(TickHandle);
		}
	}

	const TCHAR* FProtocolMultiEndpointProvider::GetEndpointDisplayName(FEndpointId EndpointId) const
	{
		Session.ReadAccessCheck();
		
		const bool bIsMainTrace = EndpointId == Session.GetTraceId();
		const FInitMessage* EndpointInitdata = bIsMainTrace
			? MainTraceInitData ? &MainTraceInitData.GetValue() : nullptr
			: EndpointInitData.Find(EndpointId);
		
		const TCHAR* EmptyString = TEXT("Null");
		return EndpointInitdata
			? EndpointInitdata->GetClientDisplayName().Get(EmptyString)
			: EmptyString;
	}

	void FProtocolMultiEndpointProvider::EnumerateProtocols(TFunctionRef<TraceServices::EEventEnumerate(FProtocolId)> Callback) const
	{
		Session.ReadAccessCheck();

		for (FProtocolId ProtocolId : ActiveProtocols)
		{
			if (Callback(ProtocolId) == TraceServices::EEventEnumerate::Stop)
			{
				break;
			}
		}
	}

	void FProtocolMultiEndpointProvider::EnumerateObjects(FProtocolId Protocol, TFunctionRef<TraceServices::EEventEnumerate(FObjectPath Object)> Callback) const
	{
		Session.ReadAccessCheck();

		for (const TPair<FObjectScopeInfo, FPerObjectData>& ObjectPair : PerObjectData)
		{
			if (ObjectPair.Key.ProtocolId == Protocol
				&& Callback(ObjectPair.Key.ObjectPath) == TraceServices::EEventEnumerate::Stop)
			{
				break;
			}
		}
	}

	void FProtocolMultiEndpointProvider::EnumerateEndpointsInSequence(const FSequenceScopeInfo& Info, TFunctionRef<TraceServices::EEventEnumerate(FEndpointId EndpointId)> Callback) const
	{
		Session.ReadAccessCheck();

		const FPerSequenceData* SequenceData = FindSequenceData(Info);
		if (!SequenceData)
		{
			return;
		}

		for (FEndpointId EndpointId : SequenceData->Endpoints)
		{
			if (Callback(EndpointId) == TraceServices::EEventEnumerate::Stop)
			{
				break;
			}
		}
	}

	void FProtocolMultiEndpointProvider::EnumerateSequences(double Start, double End, const FObjectScopeInfo& Info, TFunctionRef<TraceServices::EEventEnumerate(FSequenceId Sequence)> Callback) const
	{
		Session.ReadAccessCheck();

		const FPerObjectData* ObjectData = FindObjectData(Info);
		if (!ObjectData)
		{
			return;
		}

		// CachedTimelineSortedByStart or one of the sequence times may have changed since the last call. If so, resort.
		if (ObjectData->bIsTimelineDirty)
		{
			// USUALLY SequenceID x < y implies x happened before y but we do not enforce this (so no optimization done here)
			ObjectData->CachedTimelineSortedByStart.Sort([this, &Info](FSequenceId Left, FSequenceId Right)
			{
				const double LeftStartTime = FindSequenceData(Info.MakeSequenceInfo(Left))->Start;
				const double RightStartTime = FindSequenceData(Info.MakeSequenceInfo(Right))->Start;
				return LeftStartTime < RightStartTime;
			});
			ObjectData->bIsTimelineDirty = false;
		}

		// Find the index of first sequence that falls into the time window ...
		const int32 LowerIndex = Algo::LowerBoundBy(ObjectData->CachedTimelineSortedByStart, Start, [this, Start, End, &Info](FSequenceId SequenceId)
		{
			const FPerSequenceData* SequenceData = FindSequenceData(Info.MakeSequenceInfo(SequenceId));
			return Private::ShouldDisplayScopeInWindow(Start, End, SequenceData->Start, SequenceData->GetEndTime())
				// Causes it to be included because LowerBoundBy looks <=. Also won't interfere with earlier sequences because LowerBoundBy looks for the FIRST element <=.
				? FMath::Max(Start, SequenceData->Start)
				// Causes it to be excluded
				: SequenceData->Start;
		});
		if (LowerIndex >= ObjectData->CachedTimelineSortedByStart.Num())
		{
			return;
		}

		// ... and iterate until the first index is outside of the requested time window
		for (int32 i = LowerIndex; i < ObjectData->CachedTimelineSortedByStart.Num(); ++i)
		{
			const FSequenceId SequenceId = ObjectData->CachedTimelineSortedByStart[i];
			const FPerSequenceData* SequenceData = FindSequenceData(Info.MakeSequenceInfo(SequenceId));
			
			const bool bIsIndexInTimeWindow = SequenceData->Start <= End;
			if (!bIsIndexInTimeWindow || Callback(SequenceId) == TraceServices::EEventEnumerate::Stop)
			{
				break;
			}
		}
	}

	TOptional<FVector2d> FProtocolMultiEndpointProvider::GetSequenceBounds(const FSequenceScopeInfo& Info) const
	{
		Session.ReadAccessCheck();
		
		const FPerSequenceData* SequenceData = FindSequenceData(Info);
		if (SequenceData)
		{
			return FVector2d{ SequenceData->Start, SequenceData->GetEndTime() };
		}

		return {};
	}

	void FProtocolMultiEndpointProvider::EnumerateNetworkScopes(double Start, double End, const FSequenceScopeInfo& Info, TFunctionRef<TraceServices::EEventEnumerate(double, double, const FObjectNetworkScope&)> Callback) const
	{
		Session.ReadAccessCheck();

		const FPerSequenceData* SequenceData = FindSequenceData(Info);
		if (!SequenceData)
		{
		    return;
		}

		struct FNetworkScope
		{
		    double Start;
		    double End;
		    TOptional<FEndpointId> EndpointId;
		};

		// 1. Get all known scopes
		TArray<FNetworkScope> Scopes;

		// This is a very low number (number of analyzed .utrace files, usually <10) so it's relatively cheap to iterate through all events.
		// The problem we're solving here is that we are analyzing events from multiple .utrace files in a non-deterministic order.
		// That makes it tricky to analyse transmission scopes when the TransmissionStart, End, and Sink events are appended.
		// It is easier just to aggregate them here...
		for (int32 EventIdx = 0; EventIdx < SequenceData->NetworkScopeTimeline.GetEventCount(); ++EventIdx)
		{
			const double EventStart = SequenceData->NetworkScopeTimeline.GetEventStartTime(EventIdx);
			const double EventEnd = SequenceData->NetworkScopeTimeline.GetEventEndTime(EventIdx);
		    Scopes.Add({ EventStart, EventEnd, SequenceData->NetworkScopeTimeline.GetEvent(EventIdx) });
		}
		if (Scopes.IsEmpty())
		{
		    return;
		}

		// 2. Sort them by start time
		Scopes.Sort([](const FNetworkScope& Left, const FNetworkScope& Right){ return Left.Start < Right.Start; });

		// 3. Then fill in any gaps by interpreting them to be network transit times
		const auto CallbackIfInRange = [Start, End, &Callback](double ScopeStart, double ScopeEnd, const FObjectNetworkScope& Event)
		{
		    return Private::ShouldDisplayScopeInWindow(Start, End, ScopeStart, ScopeEnd)
		       ? Callback(ScopeStart, ScopeEnd, Event)
		       : TraceServices::EEventEnumerate::Continue;
		};

		if (CallbackIfInRange(Scopes[0].Start, Scopes[0].End, { Scopes[0].EndpointId }) == TraceServices::EEventEnumerate::Stop)
		{
		    return;
		}
		for (int32 i = 1; i < Scopes.Num(); ++i)
		{
		    const FNetworkScope& Previous = Scopes[i - 1];
		    const FNetworkScope& Current = Scopes[i];
		    
		    const double NetworkTransportTime = Current.Start - Previous.End;
		    if (LIKELY(NetworkTransportTime >= 0.0) // If it is negative, we probably received bad data.
		       && CallbackIfInRange(Previous.End, Current.Start, { {} }) == TraceServices::EEventEnumerate::Stop) 
		    {
		       return;
		    }

		    if (CallbackIfInRange(Current.Start, Current.End, { Current.EndpointId }) == TraceServices::EEventEnumerate::Stop)
		    {
		       return;
		    }
		}
	}

	void FProtocolMultiEndpointProvider::ReadProcessingStepTimeline(const FEndpointScopeInfo& Info, TFunctionRef<void(const TraceServices::ITimeline<FObjectProcessingStep>& Timeline)> Callback) const
	{
		Session.ReadAccessCheck();
		
		if (const FPerSequenceEndpointData* SequenceData = FindSequenceEndpointData(Info))
		{
			Callback(SequenceData->CpuTimeline);
		}
	}

	void FProtocolMultiEndpointProvider::AppendInit(FInitMessage Message)
	{
		Private::LogInitMessage(TEXT("Received main trace file "), Session.GetTraceId(), Message);
		
		const TraceServices::FAnalysisSessionEditScope EditScope(Session);
		if (!MainTraceInitData.IsSet())
		{
			MainTraceInitData = MoveTemp(Message);

			// Now we know the main .utrace file's start time stamp, analysis is now able compute the time offsets to the other machines. So start the analysis now. 
			Aggregator.StartAggregatedAnalysis();
		}
		else
		{
			// This can happen when you start recording in editor, stop, and start recording again.
			// TODO: Recalculate time offsets.
			UE_LOGF(LogConcertInsights, Warning, "Received init message twice for main trace file %u. Investigate.", Session.GetTraceId());
		}
	}

	bool FProtocolMultiEndpointProvider::ProcessGeneratedTraceData(float)
	{
		// This is run after Slate ticks: take a look at FUserInterfaceCommand::Run.
		// That means that all ITimingViewExtender have already be run this tick.
		// It would be better if this was run BEFORE but there's currently no integrated callback for that; FCoreDelegates::OnBeginFrame could be added to main to achieve this.
		const TraceServices::FAnalysisSessionEditScope EditScope(Session);

		Aggregator.EnumerateTraceFiles([this](uint64 TraceId)
		{
			Aggregator.ProcessEnqueuedData(TraceId, [this, TraceId](FProtocolDataQueue& DataQueue)
			{
				ProcessAggregatedTraceData(TraceId, DataQueue);
			});
			
			return TraceServices::EEventEnumerate::Continue;
		});

		return true; 
	}

	void FProtocolMultiEndpointProvider::ProcessAggregatedTraceData(FEndpointId EndpointId, FProtocolDataQueue& DataQueue)
	{
		Session.WriteAccessCheck();

		FProtocolQueuedItem Item;
		while (DataQueue.MessageQueue.Dequeue(Item))
		{
			switch (Item.Type)
			{
			case EMessageType::Init: ProcessInit(EndpointId, Item.Message.Init); break;
			case EMessageType::ObjectTraceBegin: ProcessObjectTraceBegin(EndpointId, Item.Message.ObjectTraceBegin); break;
			case EMessageType::ObjectTraceEnd: ProcessObjectTraceEnd(EndpointId, Item.Message.ObjectTraceEnd); break;
			case EMessageType::ObjectTransmissionStart: ProcessObjectTransmissionStart(EndpointId, Item.Message.TransmissionStart); break;
			case EMessageType::ObjectTransmissionReceive: ProcessObjectTransmissionReceive(EndpointId, Item.Message.TransmissionReceive); break;
			case EMessageType::ObjectSink: ProcessObjectSink(EndpointId, Item.Message.Sink); break;
				
			case EMessageType::None: [[fallthrough]];
			default:
				checkf(false, TEXT("Invalid type %u"), static_cast<uint8>(Item.Type));
			}
		}
	}

	void FProtocolMultiEndpointProvider::ProcessInit(FEndpointId EndpointId, const FInitMessage& Init)
	{
		const TraceServices::FAnalysisSessionEditScope EditScope(Session);
		if (!ensure(!EndpointInitData.Contains(EndpointId)))
		{
			UE_LOGF(LogConcertInsights, Warning, "Received init message twice for aggregated trace file %llu. Investigate.", EndpointId);
			return;
		}
		
		const bool bDuplicateEndpoint = Algo::AnyOf(EndpointInitData, [&Init](const TPair<FEndpointId, FInitMessage>& Entry)
		{
			const bool bSameEndpoint = Entry.Value.GetEndpointId().IsSet() == Init.GetEndpointId().IsSet() && *Entry.Value.GetEndpointId() == *Init.GetEndpointId();
			const bool bDuplicateServer = Entry.Value.IsServer() == Init.IsServer();
			return bSameEndpoint && bDuplicateServer;
		});
		if (bDuplicateEndpoint)
		{
			UE_CLOGF(bDuplicateEndpoint, LogConcertInsights, Warning, "Session endpoint (endpoint:%ls, bServer:%d) now encountered in trace file %llu is from was already encountered before. Investigate.",
				Init.GetEndpointId() ? *(Init.GetEndpointId()->ToString()) : TEXT("none"),
				Init.IsServer(),
				EndpointId
				);
			UE_DEBUG_BREAK();
			return;
		}

		Private::LogInitMessage(TEXT("Processed aggregated trace file "), EndpointId, Init);
		EndpointInitData.Emplace(EndpointId, Init);
	}

	void FProtocolMultiEndpointProvider::ProcessObjectTraceBegin(FEndpointId EndpointId, const FObjectTraceMessage& Message)
	{
		const TraceServices::FAnalysisSessionEditScope EditScope(Session);
		
		const FEndpointScopeInfo Info { Message.Protocol, Message.ObjectPath, Message.SequenceId, EndpointId };
		UpdateSequenceStats(Info, Message.Time);

		if (const TOptional<double> ConvertedStart = ConvertEndpointCycleToTime(Info.EndpointId, Message.Time))
		{
			UE_LOGF(LogConcertInsights, Verbose, "ObjectTraceBegin: Time: %f, EventName: %ls, Context: %ls", *ConvertedStart, Message.EventName, *Info.ToString());
			
			FEndpointCpuTimeline& CpuTimeline = FindOrAddSequenceEndpointData(Info).CpuTimeline;
			CpuTimeline.AppendBeginEvent(*ConvertedStart, { Message.EventName });
		}
	}

	void FProtocolMultiEndpointProvider::ProcessObjectTraceEnd(FEndpointId EndpointId, const FObjectTraceMessage& Message)
	{
		const TraceServices::FAnalysisSessionEditScope EditScope(Session);
		
		const FEndpointScopeInfo Info { Message.Protocol, Message.ObjectPath, Message.SequenceId, EndpointId };
		UpdateSequenceStats(Info, Message.Time);

		if (const TOptional<double> ConvertedEnd = ConvertEndpointCycleToTime(Info.EndpointId, Message.Time))
		{
			UE_LOGF(LogConcertInsights, Verbose, "ObjectTraceEnd: Time: %f, EventName: %ls, Context: %ls", *ConvertedEnd, Message.EventName, *Info.ToString());
			
			FEndpointCpuTimeline& CpuTimeline = FindOrAddSequenceEndpointData(Info).CpuTimeline;
			CpuTimeline.AppendEndEvent(*ConvertedEnd);
		}
	}

	void FProtocolMultiEndpointProvider::ProcessObjectTransmissionStart(FEndpointId EndpointId, const FObjectTransmissionStartMessage& Message)
	{
		const TraceServices::FAnalysisSessionEditScope EditScope(Session);

		const FEndpointScopeInfo Info { Message.Protocol, Message.ObjectPath, Message.SequenceId, EndpointId };
		if (const TOptional<double> ConvertedTime = ConvertEndpointCycleToTime(Info.EndpointId, Message.GetTime()))
		{
			EndOpenNetworkScope(*ConvertedTime, Info);
		}
	}

	void FProtocolMultiEndpointProvider::ProcessObjectTransmissionReceive(FEndpointId EndpointId, const FObjectTransmissionReceiveMessage& Message)
	{
		const TraceServices::FAnalysisSessionEditScope EditScope(Session);
		
		const TOptional<double> ConvertedTime = ConvertEndpointCycleToTime(EndpointId, Message.Time);
		if (!ConvertedTime)
		{
			return;
		}
		
		const FEndpointScopeInfo Info { Message.Protocol, Message.ObjectPath, Message.SequenceId, EndpointId };
		UE_LOGF(LogConcertInsights, Verbose, "ObjectTraceEnd: Time: %f, Context: %ls", *ConvertedTime, *Info.ToString());
		UpdateSequenceStats_AlreadyConverted(Info, *ConvertedTime);

		// UpdateSequenceStats allocates FPerSequenceData so Info should be mapped.
		FPerSequenceData& SequenceData = FindSequenceDataChecked(Info);
		const bool bOpenedScope = OpenNetworkScopeIfNotOpen(Info, *ConvertedTime, SequenceData);
		UE_CLOGF(!bOpenedScope, LogConcertInsights, Warning,
			"A network scope was already open when processing %ls. Was there a duplicate transmission receive or was the receive enqueued after a CPU scope was started?", *Info.ToString()
			);
	}

	void FProtocolMultiEndpointProvider::ProcessObjectSink(FEndpointId EndpointId, const FObjectSinkMessage& Message)
	{
		const TraceServices::FAnalysisSessionEditScope EditScope(Session);

		const FEndpointScopeInfo Info { Message.Protocol, Message.ObjectPath, Message.SequenceId, EndpointId };
		if (const TOptional<double> ConvertedTime = ConvertEndpointCycleToTime(Info.EndpointId, Message.Time))
		{
			EndOpenNetworkScope(*ConvertedTime, Info);
			FindSequenceData(Info)->SinkData = { *ConvertedTime, EndpointId };
		}
	}
	
	bool FProtocolMultiEndpointProvider::OpenNetworkScopeIfNotOpen(const FEndpointScopeInfo& Info, double ConvertedTime, FPerSequenceData& SequenceData)
	{
		/*
		 * Do not open a new network scope if the client that sent a sink event reports any further events.
		 * This can happen like so:
		 * {
		 *		CONCERT_TRACE_REPLICATION_OBJECT_SCOPE
		 *		CONCERT_TRACE_REPLICATION_OBJECT_SINK
		 *		// CONCERT_TRACE_REPLICATION_OBJECT_SCOPE sends at end of scope, after the sink
		 * }
		 */
		if (SequenceData.HasSequenceEnded() && SequenceData.SinkData->SinkEndpoint == Info.EndpointId)
		{
			return false;
		}
		
		FPerSequenceEndpointData& EndpointData = FindOrAddSequenceEndpointData(Info);
		const bool bHasOpenNetworkScope = EndpointData.LastScopeStartEventId.IsSet();
		if (!bHasOpenNetworkScope)
		{
			const uint64 EventId = SequenceData.NetworkScopeTimeline.AppendBeginEvent(ConvertedTime, Info.EndpointId);
			EndpointData.LastScopeStartEventId = EventId;
			return true;
		}
		return false;
	}
	
	void FProtocolMultiEndpointProvider::EndOpenNetworkScope(double ConvertedTime, const FEndpointScopeInfo& Info)
	{
		UE_LOGF(LogConcertInsights, Verbose, "EndNetworkScope: Time: %f, Context: %ls", ConvertedTime, *Info.ToString());
		UpdateSequenceStats_AlreadyConverted(Info, ConvertedTime);
		
		FPerSequenceEndpointData& EndpointData = FindOrAddSequenceEndpointData(Info);
		const bool bHasOpenScope = EndpointData.LastScopeStartEventId.IsSet();
		if (bHasOpenScope)
		{
			// UpdateSequenceStats allocates FPerSequenceData so Info should be mapped.
			FPerSequenceData& SequenceData = FindSequenceDataChecked(Info);
			SequenceData.NetworkScopeTimeline.EndEvent(*EndpointData.LastScopeStartEventId, ConvertedTime);
			
			EndpointData.LastScopeStartEventId.Reset();
		}
		else
		{
			UE_LOGF(LogConcertInsights, Warning, "No network scope was open when processing %ls. Was there a duplicate transmission start?", *Info.ToString());
		}
	}

	void FProtocolMultiEndpointProvider::UpdateSequenceStats(const FEndpointScopeInfo& Info, double Time)
	{
		const TOptional<double> ConvertedTime = ConvertEndpointCycleToTime(Info.EndpointId, Time);
		if (ConvertedTime)
		{
			UpdateSequenceStats_AlreadyConverted(Info, *ConvertedTime);
		}
	}

	void FProtocolMultiEndpointProvider::UpdateSequenceStats_AlreadyConverted(const FEndpointScopeInfo& Info, double Time)
	{
		ActiveProtocols.Add(Info.ProtocolId);
		
		// Make sure there is an entry for the object data
		FPerObjectData& ObjectData = FindOrAddObjectData(Info);
		
		const bool bIsNewSequence = !PerSequenceData.Contains(Info);
		if (bIsNewSequence)
		{
			ObjectData.CachedTimelineSortedByStart.Add(Info.SequenceId);
			ObjectData.bIsTimelineDirty = true;
		}
		
		FPerSequenceData& SequenceData = FindOrAddSequenceData(Info, Time);
		const double NewStart = FMath::Min(SequenceData.Start, Time);
		const bool bChangedStart = NewStart != SequenceData.Start;
		ObjectData.bIsTimelineDirty |= bChangedStart;
		
		SequenceData.Start = NewStart;
		SequenceData.Endpoints.Add(Info.EndpointId);

		OpenNetworkScopeIfNotOpen(Info, Time, SequenceData);
	}

	TOptional<double> FProtocolMultiEndpointProvider::ConvertEndpointCycleToTime(FEndpointId Endpoint, double OtherEndpointTime) const
	{
		Session.ReadAccessCheck();
		ensureMsgf(MainTraceInitData, TEXT("Analysis of other trace files is not supposed to have started, yet."));
		
		if (Endpoint == GetMainTraceId())
		{
			return OtherEndpointTime;
		}
		
		if (const FInitMessage* OtherEndpoint = EndpointInitData.Find(Endpoint); OtherEndpoint)
		{
			// The other endpoint is the "source" timeline and we want that time relative to the main timeline (which acts as the "target" timeline)
			return TimeSyncUtils::ConvertSourceToTargetTime(
				MainTraceInitData->GetTraceInitTimeUtc(),
				OtherEndpoint->GetTraceInitTimeUtc(),
				MainTraceInitData->GetStartTime(),
				OtherEndpoint->GetStartTime(),
				OtherEndpointTime
				);
		}
		else
		{
			UE_LOGF(LogConcertInsights, Warning, "Endpoint %llu has no init data. Ignoring received event.", Endpoint);
			return {};
		}
	}
}
