// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Analysis/IProtocolDataTarget.h"
#include "Messages/InitMessage.h"
#include "Messages/ObjectSinkMessage.h"
#include "Messages/ObjectTraceMessage.h"
#include "Messages/ObjectTransmissionReceiveMessage.h"
#include "Messages/ObjectTransmissionStartMessage.h"
#include "ObjectSequence/ObjectNetworkScope.h"
#include "ObjectSequence/ObjectProcessingStep.h"
#include "TraceAggregator.h"
#include "Types/ProtocolId.h"
#include "Types/ScopeInfos.h"
#include "Types/SequenceId.h"

#include "Containers/Ticker.h"
#include "HAL/Platform.h"
#include "Model/IntervalTimeline.h"
#include "Model/MonotonicTimeline.h"
#include "Model/PointTimeline.h"
#include "Templates/Function.h"
#include "TraceServices/Model/AnalysisSession.h"

#include <limits>

namespace UE::ConcertInsightsVisualizer
{
	class FSingleProtocolEndpointProvider;
	
	/**
	 * This is the final provider that is visualized in Insights.
	 * 
	 * It aggregates multiple FSingleProtocolEndpointProvider into timelines that are then displayed.
	 * There is a main provider, which is the one for which the .utrace is opened, to which all other traces are made relative.
	 *
	 * This is what the data will be visualized as in the Insights graph:
	 * [--------------------------------------------- FObjectSequence - ActorName - Sequence 1 --------------------------------------------]
	 * [--------- FObjectNetworkScope - Client 1---------][FObjectNetworkScope - Transit][--------- FObjectNetworkScope - Client 2---------]
	 * [FObjectProcessingStep 1] [FObjectProcessingStep 2]                               [FObjectProcessingStep 1] [FObjectProcessingStep 2]
	 * Sequence 2 would go under this, etc.
	 */
	class FProtocolMultiEndpointProvider
		: public TraceServices::IProvider
		, public TraceServices::IEditableProvider
		, public IProtocolDataTarget
	{
	public:
		
		/** ID used for registration with Insights. */
		static const FName ProviderName;

		FProtocolMultiEndpointProvider(TraceServices::IAnalysisSession& Session UE_LIFETIMEBOUND);
		virtual ~FProtocolMultiEndpointProvider() override;

		/** Utility for getting the display name of a particular endpoint. Returns nullptr if invalid. */
		const TCHAR* GetEndpointDisplayName(FEndpointId EndpointId) const;

		/** Gets all protocols for which objects were traced. */
		void EnumerateProtocols(TFunctionRef<TraceServices::EEventEnumerate(FProtocolId)> Callback) const;
		/** Gets all objects for which data was traced. */
		void EnumerateObjects(FProtocolId Protocol, TFunctionRef<TraceServices::EEventEnumerate(FObjectPath Object)> Callback) const;
		/** Gets all endpoints that participate in the given sequence. */
		void EnumerateEndpointsInSequence(const FSequenceScopeInfo& Info, TFunctionRef<TraceServices::EEventEnumerate(FEndpointId EndpointId)> Callback) const;
		
		/** 1st row. Gets the timeline of all sequences for a particular object. */
		void EnumerateSequences(double Start, double End, const FObjectScopeInfo&, TFunctionRef<TraceServices::EEventEnumerate(FSequenceId Sequence)> Callback) const;

		/** @return Start and end time of the sequence relative to the main trace file. */
		TOptional<FVector2d> GetSequenceBounds(const FSequenceScopeInfo& Info) const;
		
		/** 2nd row. Lists out all network scopes that are active in the given time range. A scope is a time period in which an endpoint is doing processing or the object is in transit. */
		void EnumerateNetworkScopes(double Start, double End, const FSequenceScopeInfo& Info, TFunctionRef<TraceServices::EEventEnumerate(double StartTime, double EndTime, const FObjectNetworkScope& NetworkScope)> Callback) const;

		/** 3rd row+. Gets the timeline of all processing steps performed by a specific endpoint for a particular object sequence. */
		void ReadProcessingStepTimeline(const FEndpointScopeInfo& Info, TFunctionRef<void(const TraceServices::ITimeline<FObjectProcessingStep>& Timeline)> Callback) const;

		//~ Begin IProtocolDataTarget Interface
		virtual void AppendInit(FInitMessage Message) override;
		virtual void AppendObjectTraceBegin(FObjectTraceBeginMessage Message) override { ProcessObjectTraceBegin(Session.GetTraceId(), MoveTemp(Message)); }
		virtual void AppendObjectTraceEnd(FObjectTraceEndMessage Message) override { ProcessObjectTraceEnd(Session.GetTraceId(), MoveTemp(Message)); }
		virtual void AppendObjectTransmissionStart(FObjectTransmissionStartMessage Message) override { ProcessObjectTransmissionStart(Session.GetTraceId(), MoveTemp(Message)); }
		virtual void AppendObjectTransmissionReceive(FObjectTransmissionReceiveMessage Message) override { ProcessObjectTransmissionReceive(Session.GetTraceId(), MoveTemp(Message)); }
		virtual void AppendObjectSink(FObjectSinkMessage Message) override { ProcessObjectSink(Session.GetTraceId(), MoveTemp(Message)); }
		//~ Begin IProtocolDataTarget Interface

	private:

		using FEndpointCpuTimeline = TraceServices::TMonotonicTimeline<FObjectProcessingStep>;
		using FNetworkScopeTimeline = TraceServices::TIntervalTimeline<FEndpointId>;
		using FSequenceEventTimeline = TraceServices::TPointTimeline<FSequenceId>;
		
		struct FPerSequenceEndpointData
		{
			/**
			 * Timelines where endpoints spent CPU time.
			 * Displayed as 3rd row for a sequence.
			 */
			FEndpointCpuTimeline CpuTimeline;

			/**
			 * Result of FNetworkScopeTimeline::AppendBeginEvent for FPerSequenceData::NetworkScopeTimeline.
			 * Set when a scope has been started, unset when it has ended.
			 */
			TOptional<uint64> LastScopeStartEventId;

			FPerSequenceEndpointData(TraceServices::ILinearAllocator& Allocator)
				: CpuTimeline(Allocator)
			{}
		};

		struct FSequenceEndData
		{
			/** The time of the sequence event with the latest time processed so far. Set once the sink event has been received; the track will display an infinite time until then. */
			double End;
			/** The endpoint for which the sequence ended. */
			FEndpointId SinkEndpoint;
		};

		struct FPerSequenceData
		{
			/** The time of the sequence event with the earliest time processed so far; updated as new events come in.  */
			double Start;
			/** Set when the sequence sink has been encountered */
			TOptional<FSequenceEndData> SinkData;

			/** Endpoints that participated in this sequence */
			TSet<FEndpointId> Endpoints;

			/**
			 * Indicates the intervals at which an object was processed by an endpoint.
			 * Does NOT contain intervals for transit scopes; those are determined implicitly in EnumerateNetworkScopes.
			 */
			FNetworkScopeTimeline NetworkScopeTimeline;

			/** Gets the end time as it should be displayed: infinite if pending and the actual end time if the sequence has ended. */
			double GetEndTime() const { return SinkData ? SinkData->End : std::numeric_limits<double>::infinity(); }
			/** Whether a sink event has been received for the sequence. */
			bool HasSequenceEnded() const { return SinkData.IsSet(); }

			FPerSequenceData(TraceServices::ILinearAllocator& Allocator, double Start)
				: Start(Start)
				, NetworkScopeTimeline(Allocator)
			{}
		};
		
		struct FPerObjectData
		{
			/**
			 * When bIsTimelineDirty == false, this contains all sequences sorted by their start time.
			 * 
			 * Every time new sequence is added or the sequence bounds change (FPerSequenceData::Start or FPerSequenceData::End), this needs to be resorted.
			 * This is resorted lazily, i.e. only when EnumerateSequences is called.
			 *
			 * Context:
			 * - USUALLY sequences start a monotonous, i.e. id x < y implies that sequence x started before y, but we do not want to enforce this to make it easier for the trace API user.
			 * - OFTEN sequence are processed in order (i.e. usually we can just CachedTimelineSortedByStart.Add new sequence) but they can be processed out of order.
			 * - Asymptotically, it is cheaper to dirty this array with with an O(1) update and do one full O(nlogn) resort rather than keeping this
			 *	array sorted making each update O(n), which effectively boils down to insertion sort of O(n^2). Whether this is also better in practice is not profiled.
			 * For the above 3 points in mind, we opt for a full resort in EnumerateSequences.
			 *
			 * Mutable because const function EnumerateSequences updates this.
			 * In the perspective of the API user, the internal state does not change.
			 */
			mutable TArray<FSequenceId> CachedTimelineSortedByStart;

			/**
			 * When true, CachedTimelineSortedByStart needs to be re-sorted.
			 * This flag is set when either 1. a new sequence ID is added, or 2. a FPerSequenceData::Start or FPerSequenceData::End time value is changed.
			 *
			 * Mutable because const function EnumerateSequences updates this.
			 * In the perspective of the API user, the internal state does not change.
			 */
			mutable bool bIsTimelineDirty = false;
		};

		/** The main session this provider is for. Outlives this object. */
		TraceServices::IAnalysisSession& Session;
		/** Traces related files and exposes their data. Set once the session starts. */
		FTraceAggregator Aggregator;
		/** Handle to ProcessGeneratedTraceData. */
		const FTSTicker::FDelegateHandle TickHandle;

		/** Keeps track of all the protocols encountered so far. */
		TSet<FProtocolId> ActiveProtocols;
		
		TMap<FObjectScopeInfo, FPerObjectData> PerObjectData;
		TMap<FSequenceScopeInfo, FPerSequenceData> PerSequenceData;
		TMap<FEndpointScopeInfo, FPerSequenceEndpointData> PerSequenceEndpointData;

		/** Contains the init data found in the Session's trace. */
		TOptional<FInitMessage> MainTraceInitData;
		/** Maps .utrace file ID to the init data found in that file. The endpoint ID is the same as the .utrace file ID. Does not contain MainTraceInitData. */
		TMap<FEndpointId, FInitMessage> EndpointInitData;

		uint64 GetMainTraceId() const { return Session.GetTraceId(); }

		/** Process the data from Aggregator. This is run after Slate ticks: take a look at FUserInterfaceCommand::Run. */
		bool ProcessGeneratedTraceData(float DeltaTime);
		void ProcessAggregatedTraceData(FEndpointId EndpointId, FProtocolDataQueue& DataQueue);
		
		void ProcessInit(FEndpointId EndpointId, const FInitMessage& Init);
		void ProcessObjectTraceBegin(FEndpointId EndpointId, const FObjectTraceMessage& Message);
		void ProcessObjectTraceEnd(FEndpointId EndpointId, const FObjectTraceMessage& Message);
		void ProcessObjectTransmissionStart(FEndpointId EndpointId, const FObjectTransmissionStartMessage& Message);
		void ProcessObjectTransmissionReceive(FEndpointId EndpointId, const FObjectTransmissionReceiveMessage& Message);
		void ProcessObjectSink(FEndpointId EndpointId, const FObjectSinkMessage& Message);

		// Getters
		const FPerObjectData* FindObjectData(const FObjectScopeInfo& Info) const { return PerObjectData.Find(Info); }
		const FPerSequenceData* FindSequenceData(const FSequenceScopeInfo& Info) const { return PerSequenceData.Find(Info); }
		FPerSequenceData* FindSequenceData(const FSequenceScopeInfo& Info) { return PerSequenceData.Find(Info); }
		const FPerSequenceEndpointData* FindSequenceEndpointData(const FEndpointScopeInfo& Info) const { return PerSequenceEndpointData.Find(Info); }
		// Setters
		FPerObjectData& FindOrAddObjectData(const FObjectScopeInfo& Info) { return PerObjectData.FindOrAdd(Info); }
		FPerSequenceData& FindOrAddSequenceData(const FSequenceScopeInfo& Info, double Start) { return PerSequenceData.FindOrAdd(Info, FPerSequenceData(Session.GetLinearAllocator(), Start)); }
		FPerSequenceData& FindSequenceDataChecked(const FSequenceScopeInfo& Info) { return PerSequenceData[Info]; }
		FPerSequenceEndpointData& FindOrAddSequenceEndpointData(const FEndpointScopeInfo& Info) { return PerSequenceEndpointData.FindOrAdd(Info, FPerSequenceEndpointData(Session.GetLinearAllocator())); }

		/**
		 * Opens a network scope.
		 * 
		 * Usually opened by TransmissionReceive.
		 * 
		 * Also used whenever any message is received to check whether a scope was opened implicitly:
		 * At the beginning of a sequence, there is usually no TransmissionReceive but a scope needs to be opened.
		 *
		 * @return Whether a scope was opened
		 */
		bool OpenNetworkScopeIfNotOpen(const FEndpointScopeInfo& Info, double ConvertedTime, FPerSequenceData& SequenceData);
		/** Used by TransmissionStart or ObjectSink event to end the scope of the current network user. */
		void EndOpenNetworkScope(double ConvertedTime, const FEndpointScopeInfo& Info);
		
		/**
		 * Converts Time using ConvertEndpointCycleToTime, updates ActiveProtocols, updates the known start and end times of the sequence,
		 * and caches that events occured at the times (for speeding up search which events happened in a certain time window).
		 */
		void UpdateSequenceStats(const FEndpointScopeInfo& Info, double Time);
		/** Overload that already accepts an already converted Time (ConvertEndpointCycleToTime was already called, just store the Time that is passed in). */
		void UpdateSequenceStats_AlreadyConverted(const FEndpointScopeInfo& Info, double Time);
		
		/** Computes what time on the main .utrace file another endpoint's time corresponds to. Fails if that endpoint did not send any init event. */
		TOptional<double> ConvertEndpointCycleToTime(FEndpointId Endpoint, double OtherEndpointTime) const;
	};
}

