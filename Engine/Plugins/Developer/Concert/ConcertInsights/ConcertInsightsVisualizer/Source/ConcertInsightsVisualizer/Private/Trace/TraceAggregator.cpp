// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceAggregator.h"

#include "LogConcertInsights.h"
#include "Trace/StoreClient.h"
#include "TraceServices/AnalyzerFactories.h"

namespace UE::ConcertInsightsVisualizer
{
	FTraceAggregator::FTraceData::FTraceData(TSharedRef<TraceServices::IAnalysisSession> InSession)
		: Session(MoveTemp(InSession))
		, Analyzer(*Session, DataQueue)
	{
		AnalysisContext.AddAnalyzer(Analyzer);
	}

	FTraceAggregator::FTraceAggregator(const Trace::FStoreClient& StoreClient, uint64 MainTraceId)
		: StoreClient(StoreClient)
		, MainTraceId(MainTraceId)
	{}

	void FTraceAggregator::StartAggregatedAnalysis()
	{
		const Trace::FStoreClient::FTraceInfo* MainTraceInfo = StoreClient.GetTraceInfoById(MainTraceId);
		if (ensure(MainTraceInfo))
		{
			// For now, only non-live traces can be analyzed because there is no API for synchronizing the analyzing threads: reading from the providers would cause race conditions.
			AnalyzeCompletedTraces(*MainTraceInfo);
		}
	}

	void FTraceAggregator::EnumerateTraceFiles(TFunctionRef<TraceServices::EEventEnumerate(uint64 TraceId)> Callback) const
	{
		for (const TPair<uint64, TUniquePtr<FTraceData>>& Pair : AggregatedTraces)
		{
			if (Callback(Pair.Key) == TraceServices::EEventEnumerate::Stop)
			{
				break;
			}
		}
	}

	void FTraceAggregator::ProcessEnqueuedData(uint64 TraceId, TFunctionRef<void(FProtocolDataQueue& DataQueue)> Callback)
	{
		if (const TUniquePtr<FTraceData>* TraceData = AggregatedTraces.Find(TraceId))
		{
			Callback(TraceData->Get()->DataQueue);
		}
	}

	void FTraceAggregator::AnalyzeCompletedTraces(const FTraceCachedInfo& MainTraceInfo)
	{
		UE_LOGF(LogConcertInsights, Log, "Analyzing aggregated traces");
			
		for (uint32 TraceIndex = 0; TraceIndex < StoreClient.GetTraceCount(); ++TraceIndex)
		{
			const Trace::FStoreClient::FTraceInfo* TraceInfo = StoreClient.GetTraceInfo(TraceIndex);
			if (ensure(TraceInfo) && TraceInfo->GetId() != MainTraceInfo.Id && ShouldAnalyzeTrace(*TraceInfo, MainTraceInfo))
			{
				AnalyzeTrace(TraceInfo->GetId());
			}
		}
	}

	bool FTraceAggregator::ShouldAnalyzeTrace(const Trace::FStoreClient::FTraceInfo& ConsideredTraceInfo, const FTraceCachedInfo& MainTraceInfo)
	{
		check(ConsideredTraceInfo.GetId() != MainTraceInfo.Id);
		
		const FDateTime ConsideredTime(ConsideredTraceInfo.GetTimestamp());
		const FDateTime MainTraceTime(MainTraceInfo.Timestamp);

		// TODO DP: This is a hacky way of finding related traces. We assume they were recorded in sync.
		// In the future (maybe 5.6), we must get them by session ID but the API for that is not ready, yet.
		const FTimespan TimeDifference = ConsideredTime <= MainTraceTime
			? MainTraceTime - ConsideredTime
			: ConsideredTime - MainTraceTime;
		return TimeDifference <= FTimespan::FromSeconds(5);
	}

	void FTraceAggregator::AnalyzeTrace(uint64 TraceId)
	{
		TUniquePtr<Trace::IInDataStream> TraceStream = StoreClient.ReadTrace(TraceId);
		if (!ensure(TraceStream))
		{
			return;
		}
		UE_LOGF(LogConcertInsights, Log, "Starting analysis of aggregated trace %llu", TraceId);
		
		Trace::IInDataStream* StreamNakedPointer = TraceStream.Get();
		const TSharedPtr<TraceServices::IAnalysisSession> Session = TraceServices::CreateAnalysisSession(TraceId, TEXT("FTraceAggregator"), MoveTemp(TraceStream));
		
		TUniquePtr<FTraceData>& TraceData = AggregatedTraces.Emplace(TraceId, MakeUnique<FTraceData>(Session.ToSharedRef()));
		TraceData->AnalysisProcessor = TraceData->AnalysisContext.Process(*StreamNakedPointer);
	}
}
