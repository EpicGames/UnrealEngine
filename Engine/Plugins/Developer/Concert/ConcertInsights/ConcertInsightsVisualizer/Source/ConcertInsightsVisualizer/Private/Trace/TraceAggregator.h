// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Analysis/ProtocolDataQueue.h"
#include "Analysis/ProtocolEndpointAnalyzer.h"

#include "HAL/Platform.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Analysis.h"
#include "Trace/StoreClient.h"
#include "TraceServices/Containers/Timelines.h"
#include "TraceServices/Model/AnalysisSession.h"

namespace UE::Trace{ class FStoreClient; }

namespace UE::ConcertInsightsVisualizer
{
	struct FTraceCachedInfo
	{
		uint64 Id;
		uint64 Timestamp;
		FTraceCachedInfo(const Trace::FStoreClient::FTraceInfo& Info)
		: Id(Info.GetId())
		, Timestamp(Info.GetTimestamp())
		{}
	};
	
	/** Analyses trace files that relate to a main .utrace. */
	class FTraceAggregator : public FNoncopyable
	{
	public:

		FTraceAggregator(const Trace::FStoreClient& StoreClient UE_LIFETIMEBOUND, uint64 MainTraceId);

		/**
		 * Finds related trace files and kicks of the analyzing threads.
		 *
		 * Data will be enqueued so it can be read from the main thread.
		 * You are expected to call ProcessEnqueuedData at regular intervals from the main thread and transfer data, e.g. every tick.
		 */
		void StartAggregatedAnalysis();

		/** Enumerates all trace files currently being analyzed. */
		void EnumerateTraceFiles(TFunctionRef<TraceServices::EEventEnumerate(uint64 TraceId)> Callback) const;
		
		/** Gets the data that has been produced since the last time. */
		void ProcessEnqueuedData(uint64 TraceId, TFunctionRef<void(FProtocolDataQueue& DataQueue)> Callback);

	private:

		struct FTraceData
		{
			/**
			 * Session created for this trace file.
			 * Does not any processing (FAnalysisSession::Start not called) but simply exists for storing strings.
			 * Owns the underlying IInDataStream and keeps it alive.
			 */
			TSharedRef<TraceServices::IAnalysisSession> Session;

			/** Holds the analyzed data. This is actively dequeued */
			FProtocolDataQueue DataQueue;
			/** Fills Provider with data */
			FProtocolEndpointAnalyzer Analyzer;

			/** Context that instruments the analysis */
			Trace::FAnalysisContext AnalysisContext;
			/** Handle to thread doing the processing */
			Trace::FAnalysisProcessor AnalysisProcessor;

			FTraceData(TSharedRef<TraceServices::IAnalysisSession> InSession);
		};

		/** Client from which to obtain related trace files. */
		const Trace::FStoreClient& StoreClient;
		/** ID of the trace being aggregated into. */
		const uint64 MainTraceId;

		/** The files being analyzed */
		TMap<uint64, TUniquePtr<FTraceData>> AggregatedTraces;
		
		void AnalyzeCompletedTraces(const FTraceCachedInfo& MainTraceInfo);

		static bool ShouldAnalyzeTrace(const Trace::FStoreClient::FTraceInfo& ConsideredTraceInfo, const FTraceCachedInfo& MainTraceInfo);
		void AnalyzeTrace(uint64 TraceId);
	};
}


