// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::ConcertInsightsVisualizer
{
	/** Id of a network endpoint, which can do processing of an object. */
	using FEndpointId = uint64;
}