// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EndpointId.h"
#include "ProtocolId.h"
#include "SequenceId.h"
#include "Trace/Analysis.h"

namespace UE::ConcertInsightsVisualizer
{
	class FSingleProtocolEndpointProvider;
	using FObjectPath = FStringView;

	/** Identifies all data associated with an object */
	struct FObjectScopeInfo
	{
		FProtocolId ProtocolId;
		FObjectPath ObjectPath;

		FObjectScopeInfo() = default;
		FObjectScopeInfo(FProtocolId ProtocolId, const FObjectPath& ObjectPath)
			: ProtocolId(ProtocolId)
			, ObjectPath(ObjectPath)
		{}
		
		struct FSequenceScopeInfo MakeSequenceInfo(FSequenceId SequenceId) const;

		friend bool operator==(const FObjectScopeInfo& Left, const FObjectScopeInfo& Right) { return Left.ProtocolId == Right.ProtocolId && Left.ObjectPath == Right.ObjectPath; }
		friend bool operator!=(const FObjectScopeInfo& Left, const FObjectScopeInfo& Right) { return !(Left == Right); }
		friend uint32 GetTypeHash(const FObjectScopeInfo& Info) { return HashCombine(GetTypeHash(Info.ProtocolId), GetTypeHash(Info.ObjectPath)); }
	};
	
	/** Identifies a sequence of an object */
	struct FSequenceScopeInfo : FObjectScopeInfo
	{
		FSequenceId SequenceId;

		FSequenceScopeInfo() = default;
		FSequenceScopeInfo(FProtocolId ProtocolId, const FObjectPath& ObjectPath, FSequenceId SequenceId)
			: FObjectScopeInfo(ProtocolId, ObjectPath)
			, SequenceId(SequenceId)
		{}

		struct FEndpointScopeInfo MakeEndpointInfo(FEndpointId EndpointId) const;

		friend bool operator==(const FSequenceScopeInfo& Left, const FSequenceScopeInfo& Right) { return static_cast<const FObjectScopeInfo&>(Left) == static_cast<const FObjectScopeInfo&>(Right) && Left.SequenceId == Right.SequenceId; }
		friend bool operator!=(const FSequenceScopeInfo& Left, const FSequenceScopeInfo& Right) { return !(Left == Right); }
		friend uint32 GetTypeHash(const FSequenceScopeInfo& Info) { return HashCombine(GetTypeHash(Info.SequenceId), GetTypeHash(static_cast<const FObjectScopeInfo&>(Info))); }
	};
	
	/** Identifies an endpoint that did processing in a sequence. */
	struct FEndpointScopeInfo : FSequenceScopeInfo
	{
		FEndpointScopeInfo() = default;
		FEndpointScopeInfo(FProtocolId ProtocolId, const FObjectPath& ObjectPath, FSequenceId SequenceId, FEndpointId EndpointId)
			: FSequenceScopeInfo(ProtocolId, ObjectPath, SequenceId)
			, EndpointId(EndpointId)
		{}

		FEndpointId EndpointId;

		FString ToString() const { return FString::Printf(TEXT("Protocol: %u, Object: %s, Sequence: %u, Endpoint: %llu"), ProtocolId, ObjectPath.GetData(), SequenceId, EndpointId); }
		
		friend bool operator==(const FEndpointScopeInfo& Left, const FEndpointScopeInfo& Right) { return static_cast<const FSequenceScopeInfo&>(Left) == static_cast<const FSequenceScopeInfo&>(Right) && Left.EndpointId == Right.EndpointId; }
		friend bool operator!=(const FEndpointScopeInfo& Left, const FEndpointScopeInfo& Right) { return !(Left == Right); }
		friend uint32 GetTypeHash(const FEndpointScopeInfo& Info) { return HashCombine(GetTypeHash(Info.EndpointId), GetTypeHash(static_cast<const FSequenceScopeInfo&>(Info))); }
	};

	
	FORCEINLINE FSequenceScopeInfo FObjectScopeInfo::MakeSequenceInfo(FSequenceId SequenceId) const
	{
		return { ProtocolId, ObjectPath, SequenceId };
	}
	
	FORCEINLINE FEndpointScopeInfo FSequenceScopeInfo::MakeEndpointInfo(FEndpointId EndpointId) const
	{
		return { ProtocolId, ObjectPath, SequenceId, EndpointId };
	}
}

