// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EndpointId.h"
#include "SequenceId.h"
#include "Templates/TypeHash.h"

namespace UE::ConcertInsightsVisualizer
{
	struct FSequenceEndpointPair
	{
		FSequenceId SequenceId;
		FEndpointId EndpointId;

		friend bool operator==(const FSequenceEndpointPair& Left, const FSequenceEndpointPair& Right) { return Left.SequenceId == Right.SequenceId && Left.EndpointId == Right.EndpointId; }
		friend bool operator!=(const FSequenceEndpointPair& Left, const FSequenceEndpointPair& Right) { return !(Left == Right); }
		friend uint32 GetTypeHash(const FSequenceEndpointPair& Pair) { return HashCombine(Pair.SequenceId, Pair.EndpointId); }
	};
}
