// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimeSyncUtils.h"

#include "Misc/DateTime.h"
#include "Misc/Timespan.h"

namespace UE::ConcertInsightsVisualizer::TimeSyncUtils
{
	double ConvertSourceToTargetTime(
		const FDateTime& TargetInitUtc,
		const FDateTime& SourceInitUtc,
		double TargetInitTime,
		double SourceInitTime,
		double SourceTime
		)
	{
		/**
		 * Docu example:
		 * Global	0123456789_
		 * Target	[Init567x9]
		 * Source	-[1Init6y8]
		 * 
		 * Given the following input:
		 * - TargetInitUtc	= 22/02/2024 12:00:01 (random time I choose)
		 * - SourceInitUtc	= 22/02/2024 12:00:03 (because init event occured 2s after target's init event)
		 * - TargetInitTime = 1 (see target graph time of I)
		 * - SourceInitTime = 2 (see source graph time of I)
		 * - SourceTime		= 7 (see source graph time of y)
		 */
		
		// Going through the above example:
		// -2 = 22/02/2024 12:00:01 - 22/02/2024 12:00:03
		const FTimespan SourceToTarget_DeltaTimestamp = TargetInitUtc - SourceInitUtc;
		
		// 5 = 7 - 2 - interpretation: 5s have passed since Init event in Source
		const double SourceTimeRelativeToSourceInit = SourceTime - SourceInitTime;
		// 7 = 5 - (-2) - interpretation: everything in target happened 2s before anything in the Source timeline
		const double TargetTimeRelativeToTimeInit = SourceTimeRelativeToSourceInit - SourceToTarget_DeltaTimestamp.GetTotalSeconds();
		// 8 = 1 + 7 - interpretation: the relative times are anchored relative to the UTC times at which the init events were sent
		const double TargetTime = TargetInitTime + TargetTimeRelativeToTimeInit;
		
		return TargetTime;
	}
}