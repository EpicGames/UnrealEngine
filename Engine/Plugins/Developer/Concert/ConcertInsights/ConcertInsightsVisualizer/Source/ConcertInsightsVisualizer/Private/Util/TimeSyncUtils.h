// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

struct FDateTime;

namespace UE::ConcertInsightsVisualizer::TimeSyncUtils
{
	/**
	 * Given a source and target timeline, converts SourceTime to the equivalent time in the target timeline.
	 * Time is assumed to flow at the same pace (i.e. 1s in target timeline corresponds to 1s in source timeline).
	 *
	 * Example: Suppose in the below scenario, every character is 1 second and "[" indicates time 0 in the local timeline.
	 * Global	0123456789_
	 * Target	[Init567x9]
	 * Source	-[1Init6y8]
	 * So this function answers: Given that event y occurs in source's timeline, we want to know the value of x in target's timeline.
	 * 
	 * Given the following input:
	 * - TargetInitUtc	= 22/02/2024 12:00:01 (random time I choose)
	 * - SourceInitUtc	= 22/02/2024 12:00:03 (because init event occured 2s after target's init event)
	 * - TargetInitTime = 1 (see target graph time of I)
	 * - SourceInitTime = 2 (see source graph time of I)
	 * - SourceTime		= 7 (see source graph time of y)
	 * This yields x = ConvertSourceToTargetTime(22/02/2024 12:00:02, 22/02/2024 12:00:04, 1, 2, 7) = 8
	 * 
	 * @param TargetInitUtc UTC time of target's init event in target timeline
	 * @param SourceInitUtc UTC time of source's init event in source timeline
	 * @param TargetInitTime When the target's init event occured in the target timeline (intended in absolute seconds)
	 * @param SourceInitTime When the source's init event occured in the source timeline (intended in absolute seconds)
	 * @param SourceTime The time to convert into the target timeline (intended in absolute seconds)
	 */
	double ConvertSourceToTargetTime(
		const FDateTime& TargetInitUtc,
		const FDateTime& SourceInitUtc,
		double TargetInitTime,
		double SourceInitTime,
		double SourceTime
		);
}
